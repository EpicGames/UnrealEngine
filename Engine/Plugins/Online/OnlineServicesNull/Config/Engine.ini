[OnlineServices.EOS.Lobbies]
+ServiceDescriptors=(Id="Lobby", AttributeIds=("LobbyServiceAttribute1", "LobbyServiceAttribute2", "LobbyServiceAttribute3", "LobbyServiceAttribute4", "LobbyServiceAttribute5", "LobbyServiceAttribute6", "LobbyServiceAttribute7", "LobbyServiceAttribute8", "LobbyServiceAttribute9", "LobbyServiceAttribute10", "LobbyServiceAttribute11", "LobbyServiceAttribute12", "LobbyServiceAttribute13", "LobbyServiceAttribute14", "LobbyServiceAttribute15", "LobbyServiceAttribute16", "LobbyServiceAttribute17", "LobbyServiceAttribute18", "LobbyServiceAttribute19", "LobbyServiceAttribute20", "LobbyServiceAttribute21", "LobbyServiceAttribute22", "LobbyServiceAttribute23", "LobbyServiceAttribute24", "LobbyServiceAttribute25", "LobbyServiceAttribute26", "LobbyServiceAttribute27", "LobbyServiceAttribute28", "LobbyServiceAttribute29", "LobbyServiceAttribute30", "LobbyServiceAttribute31", "LobbyServiceAttribute32", "LobbyServiceAttribute33", "LobbyServiceAttribute34", "LobbyServiceAttribute35", "LobbyServiceAttribute36", "LobbyServiceAttribute37", "LobbyServiceAttribute38", "LobbyServiceAttribute39", "LobbyServiceAttribute40", "LobbyServiceAttribute41", "LobbyServiceAttribute42", "LobbyServiceAttribute43", "LobbyServiceAttribute44", "LobbyServiceAttribute45", "LobbyServiceAttribute46", "LobbyServiceAttribute47", "LobbyServiceAttribute48", "LobbyServiceAttribute49", "LobbyServiceAttribute50", "LobbyServiceAttribute51", "LobbyServiceAttribute52", "LobbyServiceAttribute53", "LobbyServiceAttribute54", "LobbyServiceAttribute55", "LobbyServiceAttribute56", "LobbyServiceAttribute57", "LobbyServiceAttribute58", "LobbyServiceAttribute59", "LobbyServiceAttribute60", "LobbyServiceAttribute61", "LobbyServiceAttribute62", "LobbyServiceAttribute63", "LobbyServiceAttribute64", "LobbyServiceAttribute65", "LobbyServiceAttribute66", "LobbyServiceAttribute67", "LobbyServiceAttribute68", "LobbyServiceAttribute69", "LobbyServiceAttribute70", "LobbyServiceAttribute71", "LobbyServiceAttribute72", "LobbyServiceAttribute73", "LobbyServiceAttribute74", "LobbyServiceAttribute75", "LobbyServiceAttribute76", "LobbyServiceAttribute77", "LobbyServiceAttribute78", "LobbyServiceAttribute79", "LobbyServiceAttribute80", "LobbyServiceAttribute81", "LobbyServiceAttribute82", "LobbyServiceAttribute83", "LobbyServiceAttribute84", "LobbyServiceAttribute85", "LobbyServiceAttribute86", "LobbyServiceAttribute87", "LobbyServiceAttribute88", "LobbyServiceAttribute89", "LobbyServiceAttribute90", "LobbyServiceAttribute91", "LobbyServiceAttribute92", "LobbyServiceAttribute93", "LobbyServiceAttribute94", "LobbyServiceAttribute95", "LobbyServiceAttribute96", "LobbyServiceAttribute97", "LobbyServiceAttribute98", "LobbyServiceAttribute99", "LobbyServiceAttribute100"))
+ServiceDescriptors=(Id="LobbyMember", AttributeIds=("LobbyMemberServiceAttribute1", "LobbyMemberServiceAttribute2", "LobbyMemberServiceAttribute3", "LobbyMemberServiceAttribute4", "LobbyMemberServiceAttribute5", "LobbyMemberServiceAttribute6", "LobbyMemberServiceAttribute7", "LobbyMemberServiceAttribute8", "LobbyMemberServiceAttribute9", "LobbyMemberServiceAttribute10", "LobbyMemberServiceAttribute11", "LobbyMemberServiceAttribute12", "LobbyMemberServiceAttribute13", "LobbyMemberServiceAttribute14", "LobbyMemberServiceAttribute15", "LobbyMemberServiceAttribute16", "LobbyMemberServiceAttribute17", "LobbyMemberServiceAttribute18", "LobbyMemberServiceAttribute19", "LobbyMemberServiceAttribute20", "LobbyMemberServiceAttribute21", "LobbyMemberServiceAttribute22", "LobbyMemberServiceAttribute23", "LobbyMemberServiceAttribute24", "LobbyMemberServiceAttribute25", "LobbyMemberServiceAttribute26", "LobbyMemberServiceAttribute27", "LobbyMemberServiceAttribute28", "LobbyMemberServiceAttribute29", "LobbyMemberServiceAttribute30", "LobbyMemberServiceAttribute31", "LobbyMemberServiceAttribute32", "LobbyMemberServiceAttribute33", "LobbyMemberServiceAttribute34", "LobbyMemberServiceAttribute35", "LobbyMemberServiceAttribute36", "LobbyMemberServiceAttribute37", "LobbyMemberServiceAttribute38", "LobbyMemberServiceAttribute39", "LobbyMemberServiceAttribute40", "LobbyMemberServiceAttribute41", "LobbyMemberServiceAttribute42", "LobbyMemberServiceAttribute43", "LobbyMemberServiceAttribute44", "LobbyMemberServiceAttribute45", "LobbyMemberServiceAttribute46", "LobbyMemberServiceAttribute47", "LobbyMemberServiceAttribute48", "LobbyMemberServiceAttribute49", "LobbyMemberServiceAttribute50", "LobbyMemberServiceAttribute51", "LobbyMemberServiceAttribute52", "LobbyMemberServiceAttribute53", "LobbyMemberServiceAttribute54", "LobbyMemberServiceAttribute55", "LobbyMemberServiceAttribute56", "LobbyMemberServiceAttribute57", "LobbyMemberServiceAttribute58", "LobbyMemberServiceAttribute59", "LobbyMemberServiceAttribute60", "LobbyMemberServiceAttribute61", "LobbyMemberServiceAttribute62", "LobbyMemberServiceAttribute63", "LobbyMemberServiceAttribute64", "LobbyMemberServiceAttribute65", "LobbyMemberServiceAttribute66", "LobbyMemberServiceAttribute67", "LobbyMemberServiceAttribute68", "LobbyMemberServiceAttribute69", "LobbyMemberServiceAttribute70", "LobbyMemberServiceAttribute71", "LobbyMemberServiceAttribute72", "LobbyMemberServiceAttribute73", "LobbyMemberServiceAttribute74", "LobbyMemberServiceAttribute75", "LobbyMemberServiceAttribute76", "LobbyMemberServiceAttribute77", "LobbyMemberServiceAttribute78", "LobbyMemberServiceAttribute79", "LobbyMemberServiceAttribute80", "LobbyMemberServiceAttribute81", "LobbyMemberServiceAttribute82", "LobbyMemberServiceAttribute83", "LobbyMemberServiceAttribute84", "LobbyMemberServiceAttribute85", "LobbyMemberServiceAttribute86", "LobbyMemberServiceAttribute87", "LobbyMemberServiceAttribute88", "LobbyMemberServiceAttribute89", "LobbyMemberServiceAttribute90", "LobbyMemberServiceAttribute91", "LobbyMemberServiceAttribute92", "LobbyMemberServiceAttribute93", "LobbyMemberServiceAttribute94", "LobbyMemberServiceAttribute95", "LobbyMemberServiceAttribute96", "LobbyMemberServiceAttribute97", "LobbyMemberServiceAttribute98", "LobbyMemberServiceAttribute99", "LobbyMemberServiceAttribute100"))
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute1", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute2", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute3", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute4", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute5", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute6", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute7", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute8", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute9", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute10", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute11", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute12", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute13", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute14", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute15", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute16", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute17", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute18", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute19", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute20", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute21", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute22", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute23", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute24", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute25", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute26", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute27", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute28", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute29", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute30", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute31", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute32", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute33", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute34", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute35", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute36", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute37", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute38", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute39", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute40", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute41", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute42", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute43", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute44", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute45", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute46", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute47", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute48", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute49", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute50", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute51", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute52", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute53", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute54", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute55", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute56", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute57", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute58", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute59", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute60", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute61", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute62", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute63", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute64", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute65", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute66", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute67", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute68", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute69", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute70", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute71", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute72", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute73", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute74", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute75", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute76", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute77", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute78", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute79", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute80", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute81", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute82", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute83", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute84", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute85", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute86", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute87", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute88", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute89", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute90", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute91", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute92", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute93", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute94", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute95", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute96", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute97", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute98", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute99", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyServiceAttribute100", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute1", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute2", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute3", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute4", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute5", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute6", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute7", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute8", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute9", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute10", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute11", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute12", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute13", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute14", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute15", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute16", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute17", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute18", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute19", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute20", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute21", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute22", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute23", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute24", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute25", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute26", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute27", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute28", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute29", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute30", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute31", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute32", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute33", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute34", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute35", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute36", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute37", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute38", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute39", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute40", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute41", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute42", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute43", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute44", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute45", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute46", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute47", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute48", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute49", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute50", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute51", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute52", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute53", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute54", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute55", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute56", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute57", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute58", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute59", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute60", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute61", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute62", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute63", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute64", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute65", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute66", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute67", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute68", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute69", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute70", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute71", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute72", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute73", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute74", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute75", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute76", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute77", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute78", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute79", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute80", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute81", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute82", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute83", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute84", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute85", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute86", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute87", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute88", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute89", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute90", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute91", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute92", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute93", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute94", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute95", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute96", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute97", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute98", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute99", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)
+ServiceAttributeDescriptors=(Id="LobbyMemberServiceAttribute100", SupportedTypes=("Bool", "Int64", "Double", "String"), Flags=("Searchable", "Public", "Private"), MaxSize=1000)