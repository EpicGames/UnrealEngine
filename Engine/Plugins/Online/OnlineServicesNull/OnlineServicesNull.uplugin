{
	"FileVersion": 3,

	"FriendlyName": "Online Services Null",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Online Services implementation without an external service.",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,

	"Modules": [
		{
			"Name": "OnlineServicesNull",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		}
	],
	"Plugins": [
		{
			"Name": "OnlineServices",
			"Enabled": true
		},
		{
			"Name": "OnlineSubsystem",
			"Enabled": true
		},
		{
			"Name": "OnlineSubsystemUtils",
			"Enabled": true
		}
	]
}