// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class OnlineServicesNull : ModuleRules
{
	public OnlineServicesNull(ReadOnlyTargetRules Target) : base(Target)
    {
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreOnline",
				"Sockets",
				"OnlineServicesInterface",
				"OnlineServicesCommon",
				"OnlineSubsystem"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] { 
				"OnlineBase",
				"OnlineServicesCommonEngineUtils"
			}
		);

		if (Target.bCompileAgainstApplicationCore)
		{
			PrivateDependencyModuleNames.Add("ApplicationCore");
		}
	}
}
