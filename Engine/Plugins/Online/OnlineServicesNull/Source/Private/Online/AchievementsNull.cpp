// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/AchievementsNull.h"

#include "Online/AchievementsErrors.h"
#include "Online/AuthNull.h"
#include "Online/OnlineServicesNull.h"

namespace UE::Online {

FAchievementsNull::FAchievementsNull(FOnlineServicesNull& InOwningSubsystem)
	: Super(InOwningSubsystem)
{
}

void FAchievementsNull::UpdateConfig()
{
	Super::UpdateConfig();
	TOnlineComponent::LoadConfig(Config);
}

TOnlineAsyncOpHandle<FQueryAchievementDefinitions> FAchievementsNull::QueryAchievementDefinitions(FQueryAchievementDefinitions::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryAchievementDefinitions> Op = GetOp<FQueryAchievementDefinitions>(MoveTemp(Params));

	if (!Services.Get<FAuthNull>()->IsLoggedIn(Op->GetParams().LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	bAchievementDefinitionsQueried = true;

	Op->SetResult({});
	return Op->GetHandle();
}

TOnlineResult<FGetAchievementIds> FAchievementsNull::GetAchievementIds(FGetAchievementIds::Params&& Params)
{
	if (!Services.Get<FAuthNull>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FGetAchievementIds>(Errors::InvalidUser());
	}

	if (!bAchievementDefinitionsQueried)
	{
		// Call QueryAchievementDefinitions first
		return TOnlineResult<FGetAchievementIds>(Errors::InvalidState());
	}

	FGetAchievementIds::Result Result;
	for(const FAchievementDefinition& Definition : Config.AchievementDefinitions)
	{
		Result.AchievementIds.Emplace(Definition.AchievementId);
	}
	return TOnlineResult<FGetAchievementIds>(MoveTemp(Result));
}

TOnlineResult<FGetAchievementDefinition> FAchievementsNull::GetAchievementDefinition(FGetAchievementDefinition::Params&& Params)
{
	if (!Services.Get<FAuthNull>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FGetAchievementDefinition>(Errors::InvalidUser());
	}

	if (!bAchievementDefinitionsQueried)
	{
		// Should call QueryAchievementDefinitions first
		return TOnlineResult<FGetAchievementDefinition>(Errors::InvalidState());
	}

	const FAchievementDefinition* AchievementDefinition = FindAchievementDefinition(Params.AchievementId);
	if (!AchievementDefinition)
	{
		return TOnlineResult<FGetAchievementDefinition>(Errors::NotFound());
	}

	return TOnlineResult<FGetAchievementDefinition>({ *AchievementDefinition });
}

TOnlineAsyncOpHandle<FQueryAchievementStates> FAchievementsNull::QueryAchievementStates(FQueryAchievementStates::Params&& Params)
{
	TOnlineAsyncOpRef<FQueryAchievementStates> Op = GetOp<FQueryAchievementStates>(MoveTemp(Params));

	if (!Services.Get<FAuthNull>()->IsLoggedIn(Op->GetParams().LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (!bAchievementDefinitionsQueried)
	{
		// Call QueryAchievementDefinitions first
		Op->SetError(Errors::InvalidState());
		return Op->GetHandle();
	}

	if (!AchievementStates.Contains(Op->GetParams().LocalAccountId))
	{
		FAchievementStateMap& LocalUserAchievementStates = AchievementStates.Emplace(Op->GetParams().LocalAccountId);
		for (const FAchievementDefinition& AchievementDefinition: Config.AchievementDefinitions)
		{
			FAchievementState& AchievementState = LocalUserAchievementStates.Emplace(AchievementDefinition.AchievementId);
			AchievementState.AchievementId = AchievementDefinition.AchievementId;
		}
	}

	Op->SetResult({});

	OnAchievementStatesQueried(Op->GetParams().LocalAccountId);

	return Op->GetHandle();
}

TOnlineResult<FGetAchievementState> FAchievementsNull::GetAchievementState(FGetAchievementState::Params&& Params) const
{
	if (!Services.Get<FAuthNull>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FGetAchievementState>(Errors::InvalidUser());
	}

	const FAchievementStateMap* LocalUserAchievementStates = AchievementStates.Find(Params.LocalAccountId);
	if (!LocalUserAchievementStates)
	{
		// Call QueryAchievementStates first
		return TOnlineResult<FGetAchievementState>(Errors::InvalidState());
	}

	const FAchievementState* AchievementState = LocalUserAchievementStates->Find(Params.AchievementId);
	if (!AchievementState)
	{
		return TOnlineResult<FGetAchievementState>(Errors::NotFound());
	}

	return TOnlineResult<FGetAchievementState>({*AchievementState});
}

TOnlineAsyncOpHandle<FUnlockAchievements> FAchievementsNull::UnlockAchievements(FUnlockAchievements::Params&& Params)
{
	TOnlineAsyncOpRef<FUnlockAchievements> Op = GetOp<FUnlockAchievements>(MoveTemp(Params));

	if (!Services.Get<FAuthNull>()->IsLoggedIn(Op->GetParams().LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (Op->GetParams().AchievementIds.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	FAchievementStateMap* LocalUserAchievementStates = AchievementStates.Find(Op->GetParams().LocalAccountId);
	if (!LocalUserAchievementStates)
	{
		// Call QueryAchievementStates first
		Op->SetError(Errors::InvalidState());
		return Op->GetHandle();
	}

	for (const FString& AchievementId : Op->GetParams().AchievementIds)
	{
		const FAchievementState* AchievementState = LocalUserAchievementStates->Find(AchievementId);
		if (!AchievementState)
		{
			Op->SetError(Errors::NotFound());
			return Op->GetHandle();
		}
		if (FMath::IsNearlyEqual(AchievementState->Progress, 1.0f))
		{
			Op->SetError(Errors::Achievements::AlreadyUnlocked());
			return Op->GetHandle();
		}
	}

	FDateTime UtcNow = FDateTime::UtcNow();

	for (const FString& AchievementId : Op->GetParams().AchievementIds)
	{
		FAchievementState* AchievementState = LocalUserAchievementStates->Find(AchievementId);
		if(ensure(AchievementState))
		{
			AchievementState->Progress = 1.0f;
			AchievementState->UnlockTime = UtcNow;
		}
	}

	FAchievementStateUpdated AchievementStateUpdated;
	AchievementStateUpdated.LocalAccountId = Op->GetParams().LocalAccountId;
	AchievementStateUpdated.AchievementIds = Op->GetParams().AchievementIds;
	OnAchievementStateUpdatedEvent.Broadcast(AchievementStateUpdated);

	Op->SetResult({});
	return Op->GetHandle();
}

TOnlineResult<FDisplayAchievementUI> FAchievementsNull::DisplayAchievementUI(FDisplayAchievementUI::Params&& Params)
{
	if (!Services.Get<FAuthNull>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FDisplayAchievementUI>(Errors::InvalidUser());
	}

	const FAchievementStateMap* LocalUserAchievementStates = AchievementStates.Find(Params.LocalAccountId);
	if (!LocalUserAchievementStates)
	{
		// Call QueryAchievementStates first
		return TOnlineResult<FDisplayAchievementUI>(Errors::InvalidState());
	}

	// Safe to assume they called QueryAchievementDefinitions from this point, as that is a prereq for QueryAchievementStates.

	const FAchievementDefinition* AchievementDefinition = FindAchievementDefinition(Params.AchievementId);
	if (!AchievementDefinition)
	{
		return TOnlineResult<FDisplayAchievementUI>(Errors::NotFound());
	}

	const FAchievementState& AchievementState = LocalUserAchievementStates->FindChecked(Params.AchievementId);
	UE_LOGF(LogOnlineServices, Display, "AchievementsNull: DisplayAchievementUI LocalAccountId=[%ls]", *ToLogString(Params.LocalAccountId));
	UE_LOGF(LogOnlineServices, Display, "AchievementsNull: DisplayAchievementUI AchievementDefinition=[%ls]", *ToLogString(*AchievementDefinition));
	UE_LOGF(LogOnlineServices, Display, "AchievementsNull: DisplayAchievementUI AchievementState=[%ls]", *ToLogString(AchievementState));

	return TOnlineResult<FDisplayAchievementUI>(FDisplayAchievementUI::Result());
}

const FAchievementDefinition* FAchievementsNull::FindAchievementDefinition(const FString& AchievementId) const
{
	return Config.AchievementDefinitions.FindByPredicate([&AchievementId](const FAchievementDefinition& Def) { return Def.AchievementId == AchievementId; });
}

/* UE::Online */ }
