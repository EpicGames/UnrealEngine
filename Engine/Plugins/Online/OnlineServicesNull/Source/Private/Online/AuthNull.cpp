// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/AuthNull.h"

#include "Algo/ForEach.h"
#include "Misc/Guid.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/ScopeRWLock.h"
#include "Online/OnlineServicesNull.h"
#include "SocketSubsystem.h"

#if WITH_APPLICATION_CORE
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#endif

namespace UE::Online {
// Copied from OSS Null

struct FAuthNullConfig
{
	bool bAddUserNumToNullId = false;
	bool bForceStableNullId = false;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FAuthNullConfig)
	ONLINE_STRUCT_FIELD(FAuthNullConfig, bAddUserNumToNullId),
	ONLINE_STRUCT_FIELD(FAuthNullConfig, bForceStableNullId)
END_ONLINE_STRUCT_META()

/* Meta*/ }

namespace {

FString GenerateRandomUserId(const FAuthNullConfig& Config, FPlatformUserId PlatformUserId)
{
	FString HostName;
	if(ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM))
	{
		if (!SocketSubsystem->GetHostName(HostName))
		{
			// could not get hostname, use address
			bool bCanBindAll;
			TSharedPtr<class FInternetAddr> Addr = SocketSubsystem->GetLocalHostAddr(*GLog, bCanBindAll);
			HostName = Addr->ToString(false);
		}
	}

	bool bUseStableNullId = Config.bForceStableNullId;
	FString UserSuffix;

	if (Config.bAddUserNumToNullId)
	{
		UserSuffix = FString::Printf(TEXT("-%d"), PlatformUserId.GetInternalId());
	}
	 
	if (FPlatformProcess::IsFirstInstance() && !GIsEditor)
	{
		// If we're outside the editor and know this is the first instance, use the system login id
		bUseStableNullId = true;
	}

	if (bUseStableNullId)
	{
		// Use a stable id possibly with a user num suffix
		return FString::Printf(TEXT("OSSV2-%s-%s%s"), *HostName, *FPlatformMisc::GetLoginId().ToUpper(), *UserSuffix);
	}

	// If we're not the first instance (or in the editor), return truly random id
	return FString::Printf(TEXT("OSSV2-%s-%s%s"), *HostName, *FGuid::NewGuid().ToString(), *UserSuffix);
}

TSharedRef<FAccountInfoNull> CreateAccountInfo(const FAuthNullConfig& Config, FPlatformUserId PlatformUserId)
{
	const FString DisplayId = GenerateRandomUserId(Config, PlatformUserId);
	return MakeShared<FAccountInfoNull>(FAccountInfoNull{ {
		FOnlineAccountIdRegistryNull::Get().FindOrAddAccountId(DisplayId),
		PlatformUserId,
		ELoginStatus::LoggedIn,
		{ { AccountAttributeData::DisplayName, DisplayId } }
		} });
}

/* anonymous*/ }

TSharedPtr<FAccountInfoNull> FAccountInfoRegistryNULL::Find(FPlatformUserId PlatformUserId) const
{
	return StaticCastSharedPtr<FAccountInfoNull>(Super::Find(PlatformUserId));
}

TSharedPtr<FAccountInfoNull> FAccountInfoRegistryNULL::Find(FAccountId AccountId) const
{
	return StaticCastSharedPtr<FAccountInfoNull>(Super::Find(AccountId));
}

void FAccountInfoRegistryNULL::Register(const TSharedRef<FAccountInfoNull>& AccountInfoNULL)
{
	FWriteScopeLock Lock(IndexLock);
	DoRegister(AccountInfoNULL);
}

void FAccountInfoRegistryNULL::Unregister(FAccountId AccountId)
{
	if (TSharedPtr<FAccountInfoNull> AccountInfoNULL = Find(AccountId))
	{
		FWriteScopeLock Lock(IndexLock);
		DoUnregister(AccountInfoNULL.ToSharedRef());
	}
	else
	{
		UE_LOGF(LogOnlineServices, Warning, "[FAccountInfoRegistryNULL::Unregister] Failed to find account [%ls].", *ToLogString(AccountId));
	}
}

FAuthNull::FAuthNull(FOnlineServicesNull& InServices)
	: FAuthCommon(InServices)
{
}

void FAuthNull::Initialize()
{
	FAuthCommon::Initialize();
	InitializeUsers();
}

void FAuthNull::PreShutdown()
{
	FAuthCommon::PreShutdown();
	UninitializeUsers();
}

const FAccountInfoRegistry& FAuthNull::GetAccountInfoRegistry() const
{
	return AccountInfoRegistryNULL;
}

void FAuthNull::InitializeUsers()
{
	FAuthNullConfig AuthNullConfig;
	LoadConfig(AuthNullConfig);

#if WITH_APPLICATION_CORE // for IPlatformInputDeviceMapper
	// There is no "login" for Null - all local users are initialized as "logged in".
	TArray<FPlatformUserId> Users;
	IPlatformInputDeviceMapper::Get().GetAllActiveUsers(Users);
	Algo::ForEach(Users, [&](FPlatformUserId PlatformUserId)
	{
		AccountInfoRegistryNULL.Register(CreateAccountInfo(AuthNullConfig, PlatformUserId));
	});

	// Setup hook to add new users when they become available.
	IPlatformInputDeviceMapper::Get().GetOnInputDeviceConnectionChange().AddRaw(this, &FAuthNull::OnInputDeviceConnectionChange);
#endif
}

void FAuthNull::UninitializeUsers()
{
#if WITH_APPLICATION_CORE // for IPlatformInputDeviceMapper
	IPlatformInputDeviceMapper::Get().GetOnInputDeviceConnectionChange().RemoveAll(this);
#endif
}

void FAuthNull::OnInputDeviceConnectionChange(EInputDeviceConnectionState NewConnectionState, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId)
{
	// If this is a new platform user then register an entry for them so they will be seen as "logged-in".
	if (NewConnectionState == EInputDeviceConnectionState::Connected && PlatformUserId != PLATFORMUSERID_NONE && !AccountInfoRegistryNULL.Find(PlatformUserId))
	{
		FAuthNullConfig AuthNullConfig;
		LoadConfig(AuthNullConfig);

		TSharedRef<FAccountInfoNull> AccountInfo = CreateAccountInfo(AuthNullConfig, PlatformUserId);
		AccountInfoRegistryNULL.Register(AccountInfo);
		OnAuthLoginStatusChangedEvent.Broadcast(FAuthLoginStatusChanged{ AccountInfo, ELoginStatus::LoggedIn });
	}
}

// FOnlineAccountIdRegistryNull

FOnlineAccountIdRegistryNull::FOnlineAccountIdRegistryNull()
: Registry(EOnlineServices::Null)
{
}

FOnlineAccountIdRegistryNull& FOnlineAccountIdRegistryNull::Get()
{
	static FOnlineAccountIdRegistryNull Instance;
	return Instance;
}

FAccountId FOnlineAccountIdRegistryNull::Find(const FString& AccountId) const
{
	return Registry.FindHandle(AccountId);
}

FAccountId FOnlineAccountIdRegistryNull::FindOrAddAccountId(const FString& AccountId)
{
	return Registry.FindOrAddHandle(AccountId);
}

FString FOnlineAccountIdRegistryNull::ToString(const FAccountId& AccountId) const
{
	FString Result;
	if (Registry.ValidateOnlineId(AccountId))
	{
		Result = Registry.FindIdValue(AccountId);
	}
	else
	{
		check(!AccountId.IsValid()); // Check we haven't been passed a valid handle for a different EOnlineServices.
		Result = TEXT("Invalid");
	}
	return Result;
}

FString FOnlineAccountIdRegistryNull::ToLogString(const FAccountId& AccountId) const
{
	return ToString(AccountId);
}

TArray<uint8> FOnlineAccountIdRegistryNull::ToReplicationData(const FAccountId& AccountId) const
{
	TArray<uint8> ReplicationData;
	if (Registry.ValidateOnlineId(AccountId))
	{
		const FString& AccountIdString = Registry.FindIdValue(AccountId);
		ReplicationData.SetNumUninitialized(AccountIdString.Len());
		StringToBytes(AccountIdString, ReplicationData.GetData(), ReplicationData.Num());
		UE_LOGF(LogOnlineServices, VeryVerbose, "[FOnlineAccountIdRegistryNull::ToReplicationData] StringToBytes on [%ls] returned %d len", *AccountIdString, ReplicationData.Num())
	}
	return ReplicationData;
}

FAccountId FOnlineAccountIdRegistryNull::FromReplicationData(const TArray<uint8>& ReplicationData)
{
	return FromStringData(BytesToString(ReplicationData.GetData(), ReplicationData.Num()));
}

FAccountId FOnlineAccountIdRegistryNull::FromStringData(const FString& StringData)
{
	if (StringData.Len() > 0)
	{
		return FindOrAddAccountId(StringData);
	}
	return FAccountId();
}

/* UE::Online */ }
