// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/LeaderboardsNull.h"
#include "Online/AuthNull.h"
#include "Online/OnlineServicesNull.h"

namespace UE::Online {

struct FFindLeaderboardDataByName
{
	FFindLeaderboardDataByName(const FString& InName)
		: Name(InName)
	{
	}

	bool operator()(const FLeaderboardDataNull& LeaderboardData) const
	{
		return LeaderboardData.Name == Name;
	}

	const FString& Name;
};

FLeaderboardsNull::FLeaderboardsNull(FOnlineServicesNull& InOwningSubsystem)
	: Super(InOwningSubsystem)
{
}

namespace Private
{

uint64 GetUpdatedScoreByLeaderboardDefinition(FLeaderboardDefinition* LeaderboardDefinition, uint64 OldScore, uint64 NewScore)
{
	switch (LeaderboardDefinition->UpdateMethod)
	{
	case ELeaderboardUpdateMethod::KeepBest:
		switch (LeaderboardDefinition->OrderMethod)
		{
		case ELeaderboardOrderMethod::Descending:
			return FMath::Max(NewScore, OldScore);
		case ELeaderboardOrderMethod::Ascending:
			return FMath::Min(NewScore, OldScore);
		default: checkNoEntry();
			return NewScore;
		}
	default: checkNoEntry(); // Intentional fallthrough
	case ELeaderboardUpdateMethod::Force:
		return NewScore;
	}
}

}

TOnlineAsyncOpHandle<FWriteLeaderboardScores> FLeaderboardsNull::WriteLeaderboardScores(FWriteLeaderboardScores::Params&& Params)
{
	TOnlineAsyncOpRef<FWriteLeaderboardScores> Op = GetOp<FWriteLeaderboardScores>(MoveTemp(Params));

	if (!Services.Get<FAuthNull>()->IsLoggedIn(Op->GetParams().LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FWriteLeaderboardScores>& InAsyncOp) mutable
	{
		FLeaderboardDefinition* LeaderboardDefinition = LeaderboardDefinitions.Find(InAsyncOp.GetParams().BoardName);
		if (!LeaderboardDefinition)
		{
			InAsyncOp.SetError(Errors::NotConfigured());
			return;
		}

		FLeaderboardDataNull* LeaderboardData = LeaderboardsData.FindByPredicate(FFindLeaderboardDataByName(InAsyncOp.GetParams().BoardName));
		if (!LeaderboardData)
		{
			LeaderboardData = &LeaderboardsData.Emplace_GetRef();
			LeaderboardData->Name = InAsyncOp.GetParams().BoardName;
		}

		TDoubleLinkedList<FUserScoreNull>& UserScoreList = LeaderboardData->UserScoreList;

		uint64 UpdatedScore = InAsyncOp.GetParams().Score;

		// Update score and remove from list
		TDoubleLinkedList<FUserScoreNull>::TDoubleLinkedListNode* CurrentNode = UserScoreList.GetHead();
		while (CurrentNode)
		{
			const FUserScoreNull& UserScore = CurrentNode->GetValue();
			if (UserScore.AccountId == InAsyncOp.GetParams().LocalAccountId)
			{
				UpdatedScore = Private::GetUpdatedScoreByLeaderboardDefinition(LeaderboardDefinition, UserScore.Score, InAsyncOp.GetParams().Score);
				UserScoreList.RemoveNode(CurrentNode);
				break;
			}

			CurrentNode = CurrentNode->GetNextNode();
		}

		// Insert into list by order
		FUserScoreNull UserScoreToInsert;
		UserScoreToInsert.Score = UpdatedScore;
		UserScoreToInsert.AccountId = InAsyncOp.GetParams().LocalAccountId;

		bool bInserted = false;
		CurrentNode = UserScoreList.GetHead();
		while (CurrentNode)
		{
			const FUserScoreNull& UserScore = CurrentNode->GetValue();

			bool ShouldInsert = false;
			switch (LeaderboardDefinition->OrderMethod)
			{
			case ELeaderboardOrderMethod::Descending:
				ShouldInsert = UpdatedScore > UserScore.Score;
				break;
			case ELeaderboardOrderMethod::Ascending:
				ShouldInsert = UpdatedScore < UserScore.Score;
				break;
			default: checkNoEntry();
				break;
			}
			if (ShouldInsert)
			{
				bInserted = true;
				UserScoreList.InsertNode(UserScoreToInsert, CurrentNode);
				break;
			}

			CurrentNode = CurrentNode->GetNextNode();
		}

		if (!bInserted)
		{
			UserScoreList.AddTail(UserScoreToInsert);
		}

		InAsyncOp.SetResult({});
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FReadEntriesForUsers> FLeaderboardsNull::ReadEntriesForUsers(FReadEntriesForUsers::Params&& Params)
{
	TOnlineAsyncOpRef<FReadEntriesForUsers> Op = GetOp<FReadEntriesForUsers>(MoveTemp(Params));

	if (!Services.Get<FAuthNull>()->IsLoggedIn(Op->GetParams().LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FReadEntriesForUsers>& InAsyncOp) mutable
	{
		FReadEntriesForUsers::Result Result;

		if (FLeaderboardDataNull* LeaderboardData = LeaderboardsData.FindByPredicate(FFindLeaderboardDataByName(InAsyncOp.GetParams().BoardName)))
		{
			for (const FAccountId& AccountId : InAsyncOp.GetParams().AccountIds)
			{
				uint32 Index = 0;
				TDoubleLinkedList<FUserScoreNull>::TDoubleLinkedListNode* CurrentNode = LeaderboardData->UserScoreList.GetHead();
				while (CurrentNode)
				{
					const FUserScoreNull& UserScore = CurrentNode->GetValue();
					// When needed, use map/set instead of array for AccountIds in Params, to improve performance
					if (UserScore.AccountId == AccountId)
					{
						FLeaderboardEntry& LeaderboardEntry = Result.Entries.Emplace_GetRef();
						LeaderboardEntry.AccountId = AccountId;
						LeaderboardEntry.Rank = Index;
						LeaderboardEntry.Score = UserScore.Score;
						break;
					}
					CurrentNode = CurrentNode->GetNextNode();
					++Index;
				}
			}
		}

		InAsyncOp.SetResult(MoveTemp(Result));
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

namespace Private
{

void ReadEntriesInRange(TDoubleLinkedList<FUserScoreNull>& UserScoreList, uint32 StartIndex, uint32 Limit, TArray<FLeaderboardEntry>& OutEntries)
{
	uint32 Index = 0;
	TDoubleLinkedList<FUserScoreNull>::TDoubleLinkedListNode* CurrentNode = UserScoreList.GetHead();
	while (CurrentNode)
	{
		if (Index >= StartIndex + Limit)
			break;

		if (Index >= StartIndex)
		{
			const FUserScoreNull& UserScore = CurrentNode->GetValue();
			FLeaderboardEntry& LeaderboardEntry = OutEntries.Emplace_GetRef();
			LeaderboardEntry.AccountId = UserScore.AccountId;
			LeaderboardEntry.Rank = Index;
			LeaderboardEntry.Score = UserScore.Score;
		}

		CurrentNode = CurrentNode->GetNextNode();
		++Index;
	}
}

}

TOnlineAsyncOpHandle<FReadEntriesAroundRank> FLeaderboardsNull::ReadEntriesAroundRank(FReadEntriesAroundRank::Params&& Params)
{
	TOnlineAsyncOpRef<FReadEntriesAroundRank> Op = GetOp<FReadEntriesAroundRank>(MoveTemp(Params));

	if (!Services.Get<FAuthNull>()->IsLoggedIn(Op->GetParams().LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (Op->GetParams().Limit == 0)
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FReadEntriesAroundRank>& InAsyncOp)
	{
		FReadEntriesAroundRank::Result Result;

		if (FLeaderboardDataNull* LeaderboardData = LeaderboardsData.FindByPredicate(FFindLeaderboardDataByName(InAsyncOp.GetParams().BoardName)))
		{
			Private::ReadEntriesInRange(LeaderboardData->UserScoreList, InAsyncOp.GetParams().Rank, InAsyncOp.GetParams().Limit, Result.Entries);
		}

		InAsyncOp.SetResult(MoveTemp(Result));
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FReadEntriesAroundUser> FLeaderboardsNull::ReadEntriesAroundUser(FReadEntriesAroundUser::Params&& Params)
{
	TOnlineAsyncOpRef<FReadEntriesAroundUser> Op = GetOp<FReadEntriesAroundUser>(MoveTemp(Params));

	if (!Services.Get<FAuthNull>()->IsLoggedIn(Op->GetParams().LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (Op->GetParams().Limit == 0)
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FReadEntriesAroundUser>& InAsyncOp)
	{
		FReadEntriesAroundUser::Result Result;

		if (FLeaderboardDataNull* LeaderboardData = LeaderboardsData.FindByPredicate(FFindLeaderboardDataByName(InAsyncOp.GetParams().BoardName)))
		{
			bool FoundUser = false;
			uint32 UserRank = 0;
			TDoubleLinkedList<FUserScoreNull>::TDoubleLinkedListNode* CurrentNode = LeaderboardData->UserScoreList.GetHead();
			while (CurrentNode)
			{
				const FUserScoreNull& UserScore = CurrentNode->GetValue();
				if (UserScore.AccountId == InAsyncOp.GetParams().AccountId)
				{
					FoundUser = true;
					break;
				}

				CurrentNode = CurrentNode->GetNextNode();
				++UserRank;
			}

			if (FoundUser)
			{
				int32 StartIndex = (int32)UserRank + InAsyncOp.GetParams().Offset;
				StartIndex = FMath::Max(StartIndex, 0);
				Private::ReadEntriesInRange(LeaderboardData->UserScoreList, StartIndex, InAsyncOp.GetParams().Limit, Result.Entries);
			}
		}

		InAsyncOp.SetResult(MoveTemp(Result));
	})
	.Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

/* UE::Online */ }
