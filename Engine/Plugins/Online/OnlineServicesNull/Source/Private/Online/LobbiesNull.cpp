// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/LobbiesNull.h"
#include "Online/OnlineBase.h"
#include "Online/OnlineServicesNull.h"
#include "Online/OnlineServicesCommonEngineUtils.h"
#include "Online/NboSerializerNullSvc.h"
#include "SocketSubsystem.h"
#include "UObject/CoreNet.h"
#include "Online/LobbyRegistryNull.h"

namespace UE::Online { 

FLobbyNull::FLobbyNull()
	: Data(MakeShared<FLobby>())
{
	bool bCanBindAll;
	HostAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalHostAddr(*GLog, bCanBindAll);

	// The below is a workaround for systems that set hostname to a distinct address from 127.0.0.1 on a loopback interface.
	// See e.g. https://www.debian.org/doc/manuals/debian-reference/ch05.en.html#_the_hostname_resolution
	// and http://serverfault.com/questions/363095/why-does-my-hostname-appear-with-the-address-127-0-1-1-rather-than-127-0-0-1-in
	// Since we bind to 0.0.0.0, we won't answer on 127.0.1.1, so we need to advertise ourselves as 127.0.0.1 for any other loopback address we may have.
	uint32 HostIp = 0;
	HostAddr->GetIp(HostIp); // will return in host order
	// if this address is on loopback interface, advertise it as 127.0.0.1
	if ((HostIp & 0xff000000) == 0x7f000000)
	{
		HostAddr->SetIp(0x7f000001);	// 127.0.0.1
	}

	// Guard SetPort with IsPortValid: GetPortFromNetDriver can return out-of-range values for non-IP addresses (e.g. EOS:<PUID>).
	const int32 NetDriverPort = GetPortFromNetDriver(NAME_None);
	if (HostAddr->IsPortValid(NetDriverPort))
	{
		HostAddr->SetPort(NetDriverPort);
	}

	if (HostAddr->GetPort() == 0)
	{
		HostAddr->SetPort(7777); // temp workaround
	}

	HostAddr->GetIp(HostAddrIp);
	HostAddr->GetPort(HostAddrPort);
}

FLobbiesNull::FLobbiesNull(FOnlineServicesNull& InServices)
	: FLobbiesCommon(InServices)
	, Services(InServices)
{

}

void FLobbiesNull::Initialize()
{
	FLobbiesCommon::Initialize();
}

void FLobbiesNull::PreShutdown()
{

}

void FLobbiesNull::Tick(float DeltaSeconds)
{
	LANSessionManager.Tick(DeltaSeconds);
}

TOnlineAsyncOpHandle<FCreateLobby> FLobbiesNull::CreateLobby(FCreateLobby::Params&& Params)
{
	TOnlineAsyncOpRef<FCreateLobby> Op = GetOp<FCreateLobby>(MoveTemp(Params));

	if (Params.bPresenceEnabled)
	{
		TOnlineResult<FGetPresenceLobby> GetPresenceLobbyResult = GetPresenceLobby({ Params.LocalAccountId });
		if (GetPresenceLobbyResult.IsOk())
		{
			UE_LOGF(LogTemp, Warning, "[%ls] Failed: bPresenceEnabled was set to true, but there is already a presence Lobby", UTF8_TO_TCHAR(__FUNCTION__));
			Op->SetError(Errors::InvalidState());
			return Op->GetHandle();
		}
	}

	TSharedRef<FLobbyNull> Lobby = CreateNamedLobby(Op->GetParams());
	
	int Result = UpdateLANStatus();

	// Set the game state as pending (not started)
	// Session->SessionState = EOnlineSessionState::Pending; todo: LobbyState

	if (Result != ONLINE_SUCCESS)
	{
		// Clean up the session info so we don't get into a confused state
		RemoveLobbyFromRef(Lobby);
		Op->SetError(Errors::Unknown()); //todo: error
	}
	else
	{
		if (Params.bPresenceEnabled)
		{
			PresenceLobbiesUserMap.FindOrAdd(Params.LocalAccountId, Lobby->Data->LobbyId);
		}

		//RegisterLocalPlayers(Lobby); todo: RegisterPlayers
		Op->SetResult(FCreateLobby::Result{Lobby->Data});
	}

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FFindLobbies> FLobbiesNull::FindLobbies(FFindLobbies::Params&& Params)
{
	UE_LOGF(LogOnlineServices, Verbose, "FLobbiesNull::FindLobbies");
	if (!CurrentLobbySearch.IsValid())
	{
		CurrentLobbySearch = MakeShared<FFindLobbies::Result>();
		CurrentLobbySearchHandle = GetOp<FFindLobbies>(MoveTemp(Params));

		FindLANSession();
	}

	return CurrentLobbySearchHandle->GetHandle(); // todo: errors (if current lobby search is valid)
}

TOnlineAsyncOpHandle<FJoinLobby> FLobbiesNull::JoinLobby(FJoinLobby::Params&& Params)
{
	TOnlineAsyncOpRef<FJoinLobby> Op = GetOp<FJoinLobby>(MoveTemp(Params));

	if (Params.bPresenceEnabled)
	{
		TOnlineResult<FGetPresenceLobby> GetPresenceLobbyResult = GetPresenceLobby({ Params.LocalAccountId });
		if (GetPresenceLobbyResult.IsOk())
		{
			UE_LOGF(LogTemp, Warning, "[%ls] Failed: bPresenceEnabled was set to true, but there is already a presence Lobby", UTF8_TO_TCHAR(__FUNCTION__));
			Op->SetError(Errors::InvalidState());
			return Op->GetHandle();
		}
	}

	const FName& LobbyName = Op->GetParams().LocalName;
	TSharedPtr<FLobbyNull> Lobby = GetLobby(Op->GetParams().LobbyId);
	TSharedPtr<FLobbyNull> ExistingLobby = GetNamedLobby(LobbyName);

	if (!ExistingLobby.IsValid() && Lobby.IsValid())
	{
		AddNamedLobby(Lobby.ToSharedRef(), LobbyName);

		if (Params.bPresenceEnabled)
		{
			PresenceLobbiesUserMap.FindOrAdd(Params.LocalAccountId, Lobby->Data->LobbyId);
		}
	}
	else
	{
		if (!Lobby.IsValid())
		{
			UE_LOGF(LogOnlineServices, Warning, "Failed to find a lobby with ID %d", Op->GetParams().LobbyId.GetHandle())
		}
		else
		{
			UE_LOGF(LogOnlineServices, Warning, "Lobby with name %ls already exists", *LobbyName.ToString());
		}
		Op->SetError(Errors::RequestFailure());  
		return Op->GetHandle();
	}

	Op->SetResult(FJoinLobby::Result{Lobby->Data});
	return Op->GetHandle();

}

TOnlineAsyncOpHandle<FLeaveLobby> FLobbiesNull::LeaveLobby(FLeaveLobby::Params&& Params)
{
	TOnlineAsyncOpRef<FLeaveLobby> Op = GetOp<FLeaveLobby>(MoveTemp(Params));
	RemoveLobbyFromId(Op->GetParams().LobbyId);
	
	// We'll clear the presence mapping if this was the presence session
	TOnlineResult<FIsPresenceLobby> IsPresenceLobbyResult = IsPresenceLobby({ Params.LocalAccountId, Params.LobbyId });
	if (IsPresenceLobbyResult.IsOk())
	{
		if (IsPresenceLobbyResult.GetOkValue().bIsPresenceLobby)
		{
			PresenceLobbiesUserMap.Remove(Params.LocalAccountId);
		}
	}

	Op->SetResult(FLeaveLobby::Result{});
	return Op->GetHandle();
}

TSharedRef<FLobbyNull> FLobbiesNull::CreateNamedLobby(const FCreateLobby::Params& Params)
{
	//CHECK(!NamedLobbies.Contains(Params.LocalName))
	TSharedRef<FLobbyNull> Lobby = NamedLobbies.Add(Params.LocalName, MakeShared<FLobbyNull>());
	Lobby->Data->LobbyId = FOnlineLobbyIdRegistryNull::Get().GetNext();
	Lobby->Data->OwnerAccountId = Params.LocalAccountId;
	Lobby->Data->LocalName = Params.LocalName;
	//Lobby.Data.SchemaName = ???
	Lobby->Data->JoinPolicy = ELobbyJoinPolicy::PublicAdvertised; // temp
	Lobby->Data->Attributes = Params.Attributes;
	Lobby->Data->MaxMembers = 100; // temp

	// construct the member data.
	TSharedRef<FLobbyMember> NewMember = MakeShared<FLobbyMember>();
	NewMember->AccountId = Params.LocalAccountId;
	NewMember->Attributes = Params.UserAttributes;

	Lobby->Data->Members.Add(Params.LocalAccountId, NewMember);

	AllLobbies.Add(Lobby->Data->LobbyId, Lobby);

	return Lobby;
}

uint32 FLobbiesNull::FindLANSession()
{
	UE_LOGF(LogOnlineServices, Verbose, "FLobbiesNull::FindLANSession");
	uint32 Return = ONLINE_IO_PENDING;

	// Recreate the unique identifier for this client
	GenerateNonce((uint8*)&LANSessionManager.LanNonce, 8);

	FOnValidResponsePacketDelegate ResponseDelegate = FOnValidResponsePacketDelegate::CreateRaw(this, &FLobbiesNull::OnValidResponsePacketReceived);
	FOnSearchingTimeoutDelegate TimeoutDelegate = FOnSearchingTimeoutDelegate::CreateRaw(this, &FLobbiesNull::OnLANSearchTimeout);

	FNboSerializeToBuffer Packet(LAN_BEACON_MAX_PACKET_SIZE);
	LANSessionManager.CreateClientQueryPacket(Packet, LANSessionManager.LanNonce);
	if (LANSessionManager.Search(Packet, ResponseDelegate, TimeoutDelegate) == false)
	{
		UE_LOGF(LogOnlineServices, Warning, "FLobbiesNull::FindLANSession: Failed, returning");
		Return = ONLINE_FAIL;

		FinalizeLANSearch();

		// Just trigger the delegate as having failed
		CurrentLobbySearchHandle->SetError(Errors::Unknown());
		CurrentLobbySearch = nullptr;
		CurrentLobbySearchHandle = nullptr;
	}
	else
	{
		UE_LOGF(LogOnlineServices, Verbose, "FLobbiesNull::FindLANSession: searching....");
	}


	return Return;
}

bool FLobbiesNull::NeedsToAdvertise()
{
	// todo: only advertise if we are the host
	return !NamedLobbies.IsEmpty();
}

uint32 FLobbiesNull::UpdateLANStatus()
{
	UE_LOGF(LogOnlineServices, Verbose, "FLobbiesNull::UpdateLANStatus");
	uint32 Result = ONLINE_SUCCESS;

	if ( NeedsToAdvertise() )
	{
		// set up LAN session
		if (LANSessionManager.GetBeaconState() == ELanBeaconState::NotUsingLanBeacon)
		{
			UE_LOGF(LogOnlineServices, Verbose, "FLobbiesNull::UpdateLANStatus- Hosting..");
			FOnValidQueryPacketDelegate QueryPacketDelegate = FOnValidQueryPacketDelegate::CreateRaw(this, &FLobbiesNull::OnValidQueryPacketReceived);
			if (!LANSessionManager.Host(QueryPacketDelegate))
			{
				UE_LOGF(LogOnlineServices, Warning, "FLobbiesNull::Failed to host!");
				Result = ONLINE_FAIL;

				LANSessionManager.StopLANSession();
			}
		}
	}
	else
	{
		if (LANSessionManager.GetBeaconState() != ELanBeaconState::Searching)
		{
			// Tear down the LAN beacon
			LANSessionManager.StopLANSession();
		}
	}

	return Result;
}

void FLobbiesNull::AppendLobbyToPacket(FNboSerializeToBuffer& Packet, const TSharedRef<FLobbyNull>& Lobby)
{
	using namespace NboSerializerNullSvc;

	/** Owner of the session */
	UE_LOGF(LogOnlineServices, Verbose, "AppendLobbyToPacket: Appending IP %ls", *Lobby->HostAddr->ToString(true));

	SerializeToBuffer(Packet, Lobby->Data->OwnerAccountId);
	SerializeToBuffer(Packet, Lobby->Data->Attributes);
	Packet	<< TEXT("Temp:Owner") // todo: owner name
		<< Lobby->HostAddrIp
		<< Lobby->HostAddrPort
		<< 100 // todo: num private connections
		<< 100; // todo: num public connections

	// todo: host info, per game settings
}

void FLobbiesNull::OnValidQueryPacketReceived(uint8* PacketData, int32 PacketLength, uint64 ClientNonce)
{
	UE_LOGF(LogOnlineServices, Verbose, "FLobbiesNull::OnValidQueryPacketReceived");
	// Iterate through all registered sessions and respond for each one that can be joinable
	for (TPair<FName, TSharedRef<FLobbyNull>>& Pair : NamedLobbies)
	{
		const TSharedRef<FLobbyNull>& Lobby = Pair.Value;

		// TODO: Joinability (ensure lobby is joinable

		FNboSerializeToBuffer Packet(LAN_BEACON_MAX_PACKET_SIZE);
		// Create the basic header before appending additional information
		LANSessionManager.CreateHostResponsePacket(Packet, ClientNonce);

		// Add all the session details
		AppendLobbyToPacket(Packet, Lobby);

		// Broadcast this response so the client can see us
		if (!Packet.HasOverflow())
		{
			LANSessionManager.BroadcastPacket(Packet, Packet.GetByteCount());
		}
		else
		{
			UE_LOGF(LogOnlineServices, Warning, "LAN broadcast packet overflow, cannot broadcast on LAN");
		}
	}
}

void FLobbiesNull::ReadLobbyFromPacket(FNboSerializeFromBuffer& Packet, const TSharedRef<FLobbyNull>& Session)
{
	using namespace NboSerializerNullSvc;

	UE_LOGF(LogOnlineServices, Verbose, "Reading session information from server");

	/** Owner of the session */
	FAccountId OwningAccountId;
	FString OwningUserName;
	uint32 NumOpenPrivateConnections, NumOpenPublicConnections;
	SerializeFromBuffer(Packet, OwningAccountId);
	SerializeFromBuffer(Packet, Session->Data->Attributes);
	Packet	>> OwningUserName
		>> Session->HostAddrIp
		>> Session->HostAddrPort
		>> NumOpenPrivateConnections
		>> NumOpenPublicConnections;

	Session->Data->OwnerAccountId = OwningAccountId;
	Session->HostAddr = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
	Session->HostAddr->SetIp(Session->HostAddrIp);
	Session->HostAddr->SetPort(Session->HostAddrPort);
	Session->Data->Attributes.Add(FName(TEXT("ConnectAddress")), Session->HostAddr->ToString(true));
	Session->Data->LobbyId = FOnlineLobbyIdRegistryNull::Get().GetNext(); //TODO: this should be coming from the packet, not a random GUID
	UE_LOGF(LogOnlineServices, Verbose, "ReadLobby: Got IP %ls", *Session->HostAddr->ToString(true));

	// todo: make all the members (not just the host
	TSharedRef<FLobbyMember> NewMember = MakeShared<FLobbyMember>();
	NewMember->AccountId = OwningAccountId;
	Session->Data->Members.Add(OwningAccountId, NewMember);
	AllLobbies.Add(Session->Data->LobbyId, Session);

	// todo: host info, per game settings
}

void FLobbiesNull::OnValidResponsePacketReceived(uint8* PacketData, int32 PacketLength)
{
	UE_LOGF(LogOnlineServices, Verbose, "FLobbiesNull::OnValidResponsePacketReceived");
	if (CurrentLobbySearch.IsValid())
	{
		TSharedRef<FLobbyNull> Lobby = MakeShared<FLobbyNull>();

		// todo: ping
		// this is not a correct ping, but better than nothing
		//Lobby->PingInMs = static_cast<int32>((FPlatformTime::Seconds() - SessionSearchStartInSeconds) * 1000);

		// Prepare to read data from the packet
		FNboSerializeFromBuffer Packet(PacketData, PacketLength);
		
		ReadLobbyFromPacket(Packet, Lobby);

		CurrentLobbySearch->Lobbies.Add(Lobby->Data);

		// NOTE: we don't notify until the timeout happens
	}
	else
	{
		UE_LOGF(LogOnlineServices, Warning, "Failed to create new online game settings object");
	}
}

uint32 FLobbiesNull::FinalizeLANSearch()
{
	UE_LOGF(LogOnlineServices, Verbose, "FLobbiesNull::FinalizeLANSearch");
	if (LANSessionManager.GetBeaconState() == ELanBeaconState::Searching)
	{
		LANSessionManager.StopLANSession();
	}

	return UpdateLANStatus();
}

void FLobbiesNull::OnLANSearchTimeout()
{
	UE_LOGF(LogOnlineServices, Warning, "FLobbiesNull::OnLANSearchTimeout");
	FinalizeLANSearch();

	if (CurrentLobbySearch.IsValid())
	{
		if (CurrentLobbySearch->Lobbies.Num() > 0)
		{
			UE_LOGF(LogOnlineServices, Warning, "FLobbiesNull::OnLANSearchTimeout: Had results!");
			// Allow game code to sort the servers (todo)
			//CurrentSessionSearch->SortSearchResults();
			FFindLobbies::Result Result = *CurrentLobbySearch; // copy it- not sure if moving something owned by a shared ptr is a good idea
			CurrentLobbySearchHandle->SetResult(MoveTemp(Result)); // todo: will this fire the delegates properly even though we reset the shared ptr?
			
			CurrentLobbySearch.Reset();
			CurrentLobbySearchHandle.Reset();
			return;
		}
	}
	
	UE_LOGF(LogOnlineServices, Warning, "FLobbiesNull::OnLANSearchTimeout: no results");
	// Trigger the delegate as complete
	CurrentLobbySearchHandle->SetError(Errors::Unknown());
	CurrentLobbySearch.Reset();
	CurrentLobbySearchHandle.Reset();

}

TOnlineResult<FGetJoinedLobbies> FLobbiesNull::GetJoinedLobbies(FGetJoinedLobbies::Params&& Params)
{
	FGetJoinedLobbies::Result Result;
	for (TPair<FName, TSharedRef<FLobbyNull>>& Pair : NamedLobbies)
	{
		Result.Lobbies.Add(Pair.Value->Data);
	}
	return TOnlineResult<FGetJoinedLobbies>(MoveTemp(Result));
}


} // namespace UE::Online


#undef NOT_IMPLEMENTED
