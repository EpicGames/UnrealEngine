// Copyright Epic Games, Inc. All Rights Reserved.
#include "Online/LobbyRegistryNull.h"
#include "Misc/Guid.h"

namespace UE::Online {

FOnlineLobbyIdRegistryNull& FOnlineLobbyIdRegistryNull::Get()
{
	static FOnlineLobbyIdRegistryNull Instance;
	return Instance;
}

const FLobbyId* FOnlineLobbyIdRegistryNull::Find(FString LobbyIdStr)
{
	return StringToId.Find(LobbyIdStr);
}

FLobbyId FOnlineLobbyIdRegistryNull::FindOrAdd(FString LobbyIdStr)
{
	const FLobbyId* Entry = StringToId.Find(LobbyIdStr);
	if (Entry)
	{
		return *Entry;
	}

	Ids.Add(LobbyIdStr);
	FLobbyId LobbyId(EOnlineServices::Null, Ids.Num());
	StringToId.Add(LobbyIdStr, LobbyId);

	return LobbyId;
}

UE::Online::FLobbyId FOnlineLobbyIdRegistryNull::GetNext()
{
	return FindOrAdd(FGuid::NewGuid().ToString());
}


const FString* FOnlineLobbyIdRegistryNull::GetInternal(const FLobbyId& LobbyId) const
{
	if (LobbyId.GetOnlineServicesType() == EOnlineServices::Null && LobbyId.GetHandle() <= (uint32)Ids.Num())
	{
		return &Ids[LobbyId.GetHandle()-1];
	}
	return nullptr;
}

FString FOnlineLobbyIdRegistryNull::ToString(const FLobbyId& LobbyId) const
{
	if (const FString* Id = GetInternal(LobbyId))
	{
		return *Id;
	}

	return FString(TEXT("Invalid"));
}

FString FOnlineLobbyIdRegistryNull::ToLogString(const FLobbyId& LobbyId) const
{
	return ToString(LobbyId);
}

TArray<uint8> FOnlineLobbyIdRegistryNull::ToReplicationData(const FLobbyId& LobbyId) const
{
	if (const FString* Id = GetInternal(LobbyId))
	{
		TArray<uint8> ReplicationData;
		ReplicationData.Reserve(Id->Len());
		StringToBytes(*Id, ReplicationData.GetData(), Id->Len());
		return ReplicationData;
	}

	return TArray<uint8>();;
}

FLobbyId FOnlineLobbyIdRegistryNull::FromReplicationData(const TArray<uint8>& ReplicationData)
{
	return FromStringData(BytesToString(ReplicationData.GetData(), ReplicationData.Num()));
}

FLobbyId FOnlineLobbyIdRegistryNull::FromStringData(const FString& StringData)
{
	if (StringData.Len() > 0)
	{
		return FindOrAdd(StringData);
	}
	return FLobbyId();
}

} // namespace UE::Online
