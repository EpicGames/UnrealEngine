// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesNull.h"

#include "Online/AchievementsNull.h"
#include "Online/AuthNull.h"
#include "Online/LeaderboardsNull.h"
#include "Online/LobbiesNull.h"
#include "Online/PresenceNull.h"
#include "Online/StatsNull.h"
#include "Online/TitleFileNull.h"
#include "Online/UserFileNull.h"
#include "Online/SessionsNull.h"

namespace UE::Online {

struct FNullPlatformConfig
{
public:
	FString TestId;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FNullPlatformConfig)
// example config setup:
//	ONLINE_STRUCT_FIELD(FNullPlatformConfig, TestId)
END_ONLINE_STRUCT_META()

 /*Meta*/ }

FOnlineServicesNull::FOnlineServicesNull(FName InInstanceName, FName InInstanceConfigName)
	: FOnlineServicesCommon(TEXT("Null"), InInstanceName, InInstanceConfigName)
{
}

void FOnlineServicesNull::RegisterComponents()
{
	Components.Register<FAchievementsNull>(*this);
	Components.Register<FAuthNull>(*this);
	Components.Register<FPresenceNull>(*this);
	Components.Register<FLeaderboardsNull>(*this);
	Components.Register<FLobbiesNull>(*this);
	Components.Register<FStatsNull>(*this);
	Components.Register<FTitleFileNull>(*this);
	Components.Register<FUserFileNull>(*this);
	Components.Register<FSessionsNull>(*this);
	FOnlineServicesCommon::RegisterComponents();
}

void FOnlineServicesNull::PreShutdown()
{
	FOnlineServicesCommon::PreShutdown();

	Components.Unregister<FAchievementsNull>();
	Components.Unregister<FAuthNull>();
	Components.Unregister<FPresenceNull>();
	Components.Unregister<FLeaderboardsNull>();
	Components.Unregister<FLobbiesNull>();
	Components.Unregister<FStatsNull>();
	Components.Unregister<FTitleFileNull>();
	Components.Unregister<FUserFileNull>();
	Components.Unregister<FSessionsNull>();
}

TOnlineResult<FGetResolvedConnectString> FOnlineServicesNull::GetResolvedConnectString(FGetResolvedConnectString::Params&& Params)
{
	if(Params.LobbyId.IsValid())
	{
		ILobbiesPtr LobbiesPtr = GetLobbiesInterface();
		check(LobbiesPtr);

		TOnlineResult<FGetJoinedLobbies> JoinedLobbies = LobbiesPtr->GetJoinedLobbies({ Params.LocalAccountId });
		if (JoinedLobbies.IsOk())
		{
			for (TSharedRef<const FLobby>& Lobby : JoinedLobbies.GetOkValue().Lobbies)
			{
				if (Lobby->LobbyId == Params.LobbyId)
				{
					FName Tag = FName(TEXT("ConnectAddress")); // todo: global tag
					if(Lobby->Attributes.Contains(Tag))
					{
						FGetResolvedConnectString::Result Result;
						Result.ResolvedConnectString = Lobby->Attributes[Tag].GetString();
 						return TOnlineResult<FGetResolvedConnectString>(Result);
					}
					else
					{
						continue;
					}
				}
			}

			// No matching lobby
			return TOnlineResult<FGetResolvedConnectString>(Errors::InvalidParams());
		}
		else
		{
			return TOnlineResult<FGetResolvedConnectString>(JoinedLobbies.GetErrorValue());
		}
	}
	else if (Params.SessionId.IsValid())
	{
		ISessionsPtr SessionsPtr = GetSessionsInterface();
		check(SessionsPtr);

		TOnlineResult<FGetAllSessions> JoinedSessions = SessionsPtr->GetAllSessions({ Params.LocalAccountId });
		if (JoinedSessions.IsOk())
		{
			for (TSharedRef<const ISession>& Session : JoinedSessions.GetOkValue().Sessions)
			{
				if (Session->GetSessionId() == Params.SessionId)
				{
					if (Session->GetSessionSettings().CustomSettings.Contains(CONNECT_STRING_TAG))
					{
						FGetResolvedConnectString::Result Result;
						Result.ResolvedConnectString = Session->GetSessionSettings().CustomSettings[CONNECT_STRING_TAG].Data.GetString();
						return TOnlineResult<FGetResolvedConnectString>(Result);
					}
					else
					{
						continue;
					}
				}
			}

			// No matching session
			return TOnlineResult<FGetResolvedConnectString>(Errors::InvalidParams());
		}
		else
		{
			return TOnlineResult<FGetResolvedConnectString>(JoinedSessions.GetErrorValue());
		}
	}
	else
	{
		return TOnlineResult<FGetResolvedConnectString>(Errors::InvalidParams());
	}
}

void FOnlineServicesNull::Initialize()
{
	FNullPlatformConfig NullPlatformConfig;
	LoadConfig(NullPlatformConfig);

// example config loading:
//	FTCHARToUTF8 TestId(*NullPlatformConfig.TestId);

	FOnlineServicesCommon::Initialize();
}


/* UE::Online */ }

