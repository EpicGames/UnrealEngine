// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesNullModule.h"

#include "Modules/ModuleManager.h"
#include "Online/OnlineServicesRegistry.h"
#include "Online/OnlineServicesNull.h"
#include "Online/AuthNull.h"
#include "Online/SessionsNull.h"

namespace UE::Online
{

class FOnlineServicesFactoryNull : public IOnlineServicesFactory
{
public:
	virtual ~FOnlineServicesFactoryNull() {}
	virtual TSharedPtr<IOnlineServices> Create(FName InInstanceName, FName InstanceConfigName) override
	{
		return MakeShared<FOnlineServicesNull>(InInstanceName, InstanceConfigName);
	}
protected:
};

int FOnlineServicesNullModule::GetRegistryPriority()
{
	return 0;
}

void FOnlineServicesNullModule::StartupModule()
{
	// Making sure we load the module at this point will avoid errors while cooking
	const FName OnlineServicesInterfaceModuleName = TEXT("OnlineServicesInterface");
	if (!FModuleManager::Get().IsModuleLoaded(OnlineServicesInterfaceModuleName))
	{
		FModuleManager::Get().LoadModuleChecked(OnlineServicesInterfaceModuleName);
	}

	FOnlineServicesRegistry::Get().RegisterServicesFactory(EOnlineServices::Null, MakeUnique<FOnlineServicesFactoryNull>());
	FOnlineIdRegistryRegistry::Get().RegisterAccountIdRegistry(EOnlineServices::Null, &FOnlineAccountIdRegistryNull::Get());
	FOnlineIdRegistryRegistry::Get().RegisterSessionIdRegistry(EOnlineServices::Null, &FOnlineSessionIdRegistryNull::Get());
}

void FOnlineServicesNullModule::ShutdownModule()
{
	FOnlineServicesRegistry::Get().UnregisterServicesFactory(EOnlineServices::Null);
	FOnlineIdRegistryRegistry::Get().UnregisterAccountIdRegistry(EOnlineServices::Null);
	FOnlineIdRegistryRegistry::Get().UnregisterSessionIdRegistry(EOnlineServices::Null);
}

/* UE::Online */ }

IMPLEMENT_MODULE(UE::Online::FOnlineServicesNullModule, OnlineServicesNull);
