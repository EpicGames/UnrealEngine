// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/PresenceNull.h"
#include "Online/OnlineServicesNull.h"

namespace UE::Online {

FPresenceNull::FPresenceNull(FOnlineServicesNull& InServices)
	: FPresenceCommon(InServices)
{

}

void FPresenceNull::Initialize()
{

}

void FPresenceNull::PreShutdown()
{

}

TOnlineAsyncOpHandle<FQueryPresence> FPresenceNull::QueryPresence(FQueryPresence::Params&& Params)
{

	TOnlineAsyncOpRef<FQueryPresence> Op = GetOp<FQueryPresence>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FQueryPresence>& InAsyncOp) mutable
	{
		const FQueryPresence::Params& Params = InAsyncOp.GetParams();

		if(Params.bListenToChanges)
		{
			PresenceListeners.FindOrAdd(Params.LocalAccountId).Add(Params.TargetAccountId);
		}

		if(Presences.Contains(Params.TargetAccountId))
		{
			InAsyncOp.SetResult({Presences.FindChecked(Params.TargetAccountId)});
		}
		else
		{
			InAsyncOp.SetError(Errors::NotFound());
		}
	})
    .Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineResult<FGetCachedPresence> FPresenceNull::GetCachedPresence(FGetCachedPresence::Params&& Params)
{
	if (Presences.Contains(Params.LocalAccountId))
	{
		return TOnlineResult<FGetCachedPresence>({Presences.FindChecked(Params.LocalAccountId)});
	}
	else
	{
		return TOnlineResult<FGetCachedPresence>(Errors::NotFound());
	}
}

TOnlineAsyncOpHandle<FUpdatePresence> FPresenceNull::UpdatePresence(FUpdatePresence::Params&& Params)
{

	TOnlineAsyncOpRef<FUpdatePresence> Op = GetOp<FUpdatePresence>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FUpdatePresence>& InAsyncOp) mutable
	{
		const FUpdatePresence::Params& Params = InAsyncOp.GetParams();
		Presences.Add(Params.LocalAccountId, Params.Presence);

		for (const TPair<FAccountId, TSet<FAccountId>>& Pairs : PresenceListeners)
		{
			if (Pairs.Value.Contains(Params.LocalAccountId))
			{
				OnPresenceUpdatedEvent.Broadcast({Pairs.Key, Params.Presence});
			}
		}

		InAsyncOp.SetResult({});
	})
    .Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FPartialUpdatePresence> FPresenceNull::PartialUpdatePresence(FPartialUpdatePresence::Params&& Params)
{

	TOnlineAsyncOpRef<FPartialUpdatePresence> Op = GetOp<FPartialUpdatePresence>(MoveTemp(Params));

	Op->Then([this](TOnlineAsyncOp<FPartialUpdatePresence>& InAsyncOp) mutable
	{
		TSharedRef<FUserPresence> NewPresence = MakeShared<FUserPresence>();
		const FPartialUpdatePresence::Params::FMutations& Mutations = InAsyncOp.GetParams().Mutations;
		
		if (const TSharedRef<const FUserPresence>* const ExistingPresence = Presences.Find(InAsyncOp.GetParams().LocalAccountId))
		{
			*NewPresence = **ExistingPresence;
		}
		else
		{
			NewPresence->AccountId = InAsyncOp.GetParams().LocalAccountId;
		}

		TSharedRef<FUserPresence> MutatedPresence = ApplyPresenceMutations(*NewPresence, Mutations);
		Presences.Add(InAsyncOp.GetParams().LocalAccountId, MutatedPresence);

		for (const TPair<FAccountId, TSet<FAccountId>>& Pairs : PresenceListeners)
		{
			if (Pairs.Value.Contains(InAsyncOp.GetParams().LocalAccountId))
			{
				OnPresenceUpdatedEvent.Broadcast({ Pairs.Key, MutatedPresence });
			}
		}

		InAsyncOp.SetResult({});
	})
    .Enqueue(GetSerialQueue());

	return Op->GetHandle();
}

} //namespace UE::Online
