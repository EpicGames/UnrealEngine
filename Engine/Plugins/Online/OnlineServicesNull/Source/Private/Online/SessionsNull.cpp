// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/SessionsNull.h"

#include "Online/OnlineServicesNull.h"
#include "Online/NboSerializerNullSvc.h"

namespace UE::Online {

/** FOnlineSessionIdRegistryNull */

FOnlineSessionIdRegistryNull::FOnlineSessionIdRegistryNull()
	: FOnlineSessionIdRegistryLAN(EOnlineServices::Null)
{
}

FOnlineSessionIdRegistryNull& FOnlineSessionIdRegistryNull::Get()
{
	static FOnlineSessionIdRegistryNull Instance;
	return Instance;
}

/** FSessionsNull */

FSessionsNull::FSessionsNull(FOnlineServicesNull& InServices)
	: Super(InServices)
{

}

/** FSessionsLAN */

void FSessionsNull::AppendSessionToPacket(FNboSerializeToBuffer& Packet, const FSessionLAN& Session)
{
	using namespace NboSerializerLANSvc;
	using namespace NboSerializerNullSvc;

	SerializeToBuffer(Packet, Session);
	SerializeToBuffer(Packet, Session.OwnerAccountId);
	SerializeToBuffer(Packet, Session.SessionInfo.SessionId);
	SerializeToBuffer(Packet, Session.SessionMembers);
}

void FSessionsNull::ReadSessionFromPacket(FNboSerializeFromBuffer& Packet, FSessionLAN& Session)
{
	using namespace NboSerializerLANSvc;
	using namespace NboSerializerNullSvc;

	SerializeFromBuffer(Packet, Session);
	SerializeFromBuffer(Packet, Session.OwnerAccountId);
	SerializeFromBuffer(Packet, Session.SessionInfo.SessionId);
	SerializeFromBuffer(Packet, Session.SessionMembers);
}

/* UE::Online */ }