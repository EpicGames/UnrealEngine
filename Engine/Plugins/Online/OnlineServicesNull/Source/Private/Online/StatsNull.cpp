// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/StatsNull.h"
#include "Online/OnlineServicesNull.h"

namespace UE::Online {

FStatsNull::FStatsNull(FOnlineServicesNull& InOwningSubsystem)
	: Super(InOwningSubsystem)
{
}

/* UE::Online */ }
