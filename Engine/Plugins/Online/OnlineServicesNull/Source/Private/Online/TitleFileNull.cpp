// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/TitleFileNull.h"

#include "Online/AuthNull.h"
#include "Online/OnlineServicesNull.h"

namespace UE::Online {

struct FTitleFileNullFileDefinition
{
	FString Name;
	FString Contents;
};

struct FTitleFileNullConfig
{
	TArray<FTitleFileNullFileDefinition> Files;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FTitleFileNullFileDefinition)
	ONLINE_STRUCT_FIELD(FTitleFileNullFileDefinition, Name),
	ONLINE_STRUCT_FIELD(FTitleFileNullFileDefinition, Contents)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FTitleFileNullConfig)
	ONLINE_STRUCT_FIELD(FTitleFileNullConfig, Files)
END_ONLINE_STRUCT_META()

/* Meta */ }

FTitleFileNull::FTitleFileNull(FOnlineServicesNull& InOwningSubsystem)
	: Super(InOwningSubsystem)
{
}

void FTitleFileNull::UpdateConfig()
{
	Super::UpdateConfig();

	FTitleFileNullConfig Config;
	TOnlineComponent::LoadConfig(Config);

	TitleFiles.Reset();

	for (FTitleFileNullFileDefinition& File : Config.Files)
	{
		const FTCHARToUTF8 FileContentsUtf8(*File.Contents);
		TitleFiles.Emplace(MoveTemp(File.Name), MakeShared<FTitleFileContents>((uint8*)FileContentsUtf8.Get(), FileContentsUtf8.Length()));
	}
}

TOnlineAsyncOpHandle<FTitleFileEnumerateFiles> FTitleFileNull::EnumerateFiles(FTitleFileEnumerateFiles::Params&& Params)
{
	TOnlineAsyncOpRef<FTitleFileEnumerateFiles> Op = GetOp<FTitleFileEnumerateFiles>(MoveTemp(Params));

	if (!Services.Get<FAuthNull>()->IsLoggedIn(Op->GetParams().LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	bEnumerated = true;

	Op->SetResult({});
	return Op->GetHandle();
}

TOnlineResult<FTitleFileGetEnumeratedFiles> FTitleFileNull::GetEnumeratedFiles(FTitleFileGetEnumeratedFiles::Params&& Params)
{
	if (!Services.Get<FAuthNull>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FTitleFileGetEnumeratedFiles>(Errors::InvalidUser());
	}

	if (!bEnumerated)
	{
		// Need to call EnumerateFiles first.
		return TOnlineResult<FTitleFileGetEnumeratedFiles>(Errors::InvalidState());
	}

	FTitleFileGetEnumeratedFiles::Result Result;
	TitleFiles.GenerateKeyArray(Result.Filenames);
	return TOnlineResult<FTitleFileGetEnumeratedFiles>(MoveTemp(Result));
	
}

TOnlineAsyncOpHandle<FTitleFileReadFile> FTitleFileNull::ReadFile(FTitleFileReadFile::Params&& Params)
{
	TOnlineAsyncOpRef<FTitleFileReadFile> Op = GetOp<FTitleFileReadFile>(MoveTemp(Params));

	if (!Services.Get<FAuthNull>()->IsLoggedIn(Op->GetParams().LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (Op->GetParams().Filename.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	const FTitleFileContentsRef* Found = TitleFiles.Find(Op->GetParams().Filename);
	if (!Found)
	{
		Op->SetError(Errors::NotFound());
		return Op->GetHandle();
	}

	Op->SetResult({*Found});
	return Op->GetHandle();
}

/* UE::Online */ }
