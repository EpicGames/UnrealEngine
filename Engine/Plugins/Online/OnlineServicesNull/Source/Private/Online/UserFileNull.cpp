// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/UserFileNull.h"

#include "Misc/ConfigCacheIni.h"
#include "Online/AuthNull.h"
#include "Online/OnlineServicesNull.h"

namespace UE::Online {

FUserFileNull::FUserFileNull(FOnlineServicesNull& InOwningSubsystem)
	: Super(InOwningSubsystem)
{
}

void FUserFileNull::UpdateConfig()
{
	const TCHAR* ConfigSection = TEXT("OnlineServices.Null.UserFile");

	InitialFileStateFromConfig.Reset();

	for (int FileIdx = 0;; FileIdx++)
	{
		FString Filename;
		GConfig->GetString(ConfigSection, *FString::Printf(TEXT("File_%d_Name"), FileIdx), Filename, GEngineIni);
		if (Filename.IsEmpty())
		{
			break;
		}

		FString FileContentsStr;
		GConfig->GetString(ConfigSection, *FString::Printf(TEXT("File_%d_Contents"), FileIdx), FileContentsStr, GEngineIni);

		if (!FileContentsStr.IsEmpty())
		{
			const int ArrayLen = FileContentsStr.GetCharArray().Num() * sizeof(TCHAR);
			FUserFileContents FileContents((uint8*)FileContentsStr.GetCharArray().GetData(), ArrayLen);

			InitialFileStateFromConfig.Emplace(MoveTemp(Filename), MakeShared<const FUserFileContents>(MoveTemp(FileContents)));
		}
	}
}

TOnlineAsyncOpHandle<FUserFileEnumerateFiles> FUserFileNull::EnumerateFiles(FUserFileEnumerateFiles::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileEnumerateFiles> Op = GetOp<FUserFileEnumerateFiles>(MoveTemp(InParams));
	const FUserFileEnumerateFiles::Params& Params = Op->GetParams();

	if (!Services.Get<FAuthNull>()->IsLoggedIn(Params.LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	FUserState& UserState = GetUserState(Params.LocalAccountId);
	UserState.bEnumerated = true;

	Op->SetResult({});
	return Op->GetHandle();
}

TOnlineResult<FUserFileGetEnumeratedFiles> FUserFileNull::GetEnumeratedFiles(FUserFileGetEnumeratedFiles::Params&& Params)
{
	if (!Services.Get<FAuthNull>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FUserFileGetEnumeratedFiles>(Errors::InvalidUser());
	}

	FUserState& FileState = GetUserState(Params.LocalAccountId);
	if (!FileState.bEnumerated)
	{
		// Need to call EnumerateFiles first.
		return TOnlineResult<FUserFileGetEnumeratedFiles>(Errors::InvalidState());
	}

	FUserFileGetEnumeratedFiles::Result Result;
	FileState.Files.GenerateKeyArray(Result.Filenames);
	return TOnlineResult<FUserFileGetEnumeratedFiles>(MoveTemp(Result));
}

TOnlineAsyncOpHandle<FUserFileReadFile> FUserFileNull::ReadFile(FUserFileReadFile::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileReadFile> Op = GetOp<FUserFileReadFile>(MoveTemp(InParams));
	const FUserFileReadFile::Params& Params = Op->GetParams();

	if (!Services.Get<FAuthNull>()->IsLoggedIn(Params.LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (Params.Filename.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	FUserState& UserState = GetUserState(Params.LocalAccountId);
	const FUserFileContentsRef* Found = UserState.Files.Find(Params.Filename);
	if (!Found)
	{
		Op->SetError(Errors::NotFound());
		return Op->GetHandle();
	}

	Op->SetResult({*Found});
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FUserFileWriteFile> FUserFileNull::WriteFile(FUserFileWriteFile::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileWriteFile> Op = GetOp<FUserFileWriteFile>(MoveTemp(InParams));
	const FUserFileWriteFile::Params& Params = Op->GetParams();

	if (!Services.Get<FAuthNull>()->IsLoggedIn(Params.LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}	

	if (Params.Filename.IsEmpty() || Params.FileContents.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	FUserState& UserState = GetUserState(Params.LocalAccountId);
	UserState.Files.Emplace(Params.Filename, MakeShared<FUserFileContents>(Params.FileContents));
	Op->SetResult({});
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FUserFileCopyFile> FUserFileNull::CopyFile(FUserFileCopyFile::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileCopyFile> Op = GetOp<FUserFileCopyFile>(MoveTemp(InParams));
	const FUserFileCopyFile::Params& Params = Op->GetParams();

	if (!Services.Get<FAuthNull>()->IsLoggedIn(Params.LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (Params.SourceFilename.IsEmpty() || Params.TargetFilename.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	FUserState& UserState = GetUserState(Params.LocalAccountId);
	const FUserFileContentsRef* SourceFile = UserState.Files.Find(Params.SourceFilename);
	if (!SourceFile)
	{
		Op->SetError(Errors::NotFound());
		return Op->GetHandle();
	}

	UserState.Files.Emplace(Params.TargetFilename, *SourceFile);
	Op->SetResult({});
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FUserFileDeleteFile> FUserFileNull::DeleteFile(FUserFileDeleteFile::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileDeleteFile> Op = GetOp<FUserFileDeleteFile>(MoveTemp(InParams));
	const FUserFileDeleteFile::Params& Params = Op->GetParams();

	if (!Services.Get<FAuthNull>()->IsLoggedIn(Params.LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	if (Params.Filename.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	FUserState& UserState = GetUserState(Params.LocalAccountId);
	const bool bRemoved = UserState.Files.Remove(Params.Filename) != 0;
	if (!bRemoved)
	{
		Op->SetError(Errors::NotFound());
		return Op->GetHandle();
	}

	Op->SetResult({});
	return Op->GetHandle();
}

FUserFileNull::FUserState& FUserFileNull::GetUserState(const FAccountId AccountId)
{
	check(AccountId.IsValid());

	FUserState* FileState = UserStates.Find(AccountId);
	if (!FileState)
	{
		FileState = &UserStates.Emplace(AccountId);
		FileState->Files = InitialFileStateFromConfig;
	}

	return *FileState;
}

/* UE::Online */ }
