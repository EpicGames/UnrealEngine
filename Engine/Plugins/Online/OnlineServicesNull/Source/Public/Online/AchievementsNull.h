// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Online/AchievementsCommon.h"

#define UE_API ONLINESERVICESNULL_API

namespace UE::Online {

class FOnlineServicesNull;

struct FAchievementsNullConfig
{
	TArray<FAchievementDefinition> AchievementDefinitions;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FAchievementsNullConfig)
	ONLINE_STRUCT_FIELD(FAchievementsNullConfig, AchievementDefinitions)
END_ONLINE_STRUCT_META()

/* Meta */ }

class FAchievementsNull : public FAchievementsCommon
{
public:
	using Super = FAchievementsCommon;

	UE_API FAchievementsNull(FOnlineServicesNull& InOwningSubsystem);

	// IOnlineComponent
	UE_API virtual void UpdateConfig() override;

	// IAchievements
	UE_API virtual TOnlineAsyncOpHandle<FQueryAchievementDefinitions> QueryAchievementDefinitions(FQueryAchievementDefinitions::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetAchievementIds> GetAchievementIds(FGetAchievementIds::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetAchievementDefinition> GetAchievementDefinition(FGetAchievementDefinition::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FQueryAchievementStates> QueryAchievementStates(FQueryAchievementStates::Params&& Params) override;
	UE_API virtual TOnlineResult<FGetAchievementState> GetAchievementState(FGetAchievementState::Params&& Params) const override;
	UE_API virtual TOnlineAsyncOpHandle<FUnlockAchievements> UnlockAchievements(FUnlockAchievements::Params&& Params) override;
	UE_API virtual TOnlineResult<FDisplayAchievementUI> DisplayAchievementUI(FDisplayAchievementUI::Params&& Params) override;

protected:
	using FAchievementDefinitionMap = TMap<FString, FAchievementDefinition>;
	using FAchievementStateMap = TMap<FString, FAchievementState>;

	bool bAchievementDefinitionsQueried = false;

	FAchievementsNullConfig Config;

	UE_API const FAchievementDefinition* FindAchievementDefinition(const FString& AchievementId) const;
};

/* UE::Online */ }

#undef UE_API
