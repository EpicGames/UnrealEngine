// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "Online/AuthCommon.h"
#include "Online/OnlineIdCommon.h"

#define UE_API ONLINESERVICESNULL_API

namespace UE::Online {

class FOnlineServicesNull;

class FOnlineAccountIdRegistryNull
	: public IOnlineAccountIdRegistry
{
public:
	ONLINESERVICESNULL_API static FOnlineAccountIdRegistryNull& Get();

	ONLINESERVICESNULL_API FAccountId Find(const FString& AccountId) const;
	ONLINESERVICESNULL_API FAccountId FindOrAddAccountId(const FString& AccountId);

	// Begin IOnlineAccountIdRegistry
	virtual FString ToString(const FAccountId& AccountId) const override;
	virtual FString ToLogString(const FAccountId& AccountId) const override;
	virtual TArray<uint8> ToReplicationData(const FAccountId& AccountId) const override;
	virtual FAccountId FromReplicationData(const TArray<uint8>& ReplicationString) override;
	virtual FAccountId FromStringData(const FString& StringData) override;
	// End IOnlineAccountIdRegistry

	virtual ~FOnlineAccountIdRegistryNull() = default;

private:
	FOnlineAccountIdRegistryNull();
	TOnlineBasicAccountIdRegistry<FString> Registry;
};


struct FAccountInfoNull final : public FAccountInfo
{
};

// Auth NULL is implemented in a way similar to console platforms where there is not an explicit
// login / logout from online services. On those platforms the user account is picked either before
// the game has started or as a part of selecting an input device.
class FAccountInfoRegistryNULL final : public FAccountInfoRegistry
{
public:
	using Super = FAccountInfoRegistry;

	virtual ~FAccountInfoRegistryNULL() = default;

	UE_API TSharedPtr<FAccountInfoNull> Find(FPlatformUserId PlatformUserId) const;
	UE_API TSharedPtr<FAccountInfoNull> Find(FAccountId AccountIdHandle) const;

	UE_API void Register(const TSharedRef<FAccountInfoNull>&UserAuthData);
	UE_API void Unregister(FAccountId AccountId);
};

class FAuthNull : public FAuthCommon
{
public:
	using Super = FAuthCommon;

	UE_API FAuthNull(FOnlineServicesNull& InOwningSubsystem);
	UE_API virtual void Initialize() override;
	UE_API virtual void PreShutdown() override;

protected:
	UE_API virtual const FAccountInfoRegistry& GetAccountInfoRegistry() const override;

	UE_API void InitializeUsers();
	UE_API void UninitializeUsers();
	UE_API void OnInputDeviceConnectionChange(EInputDeviceConnectionState NewConnectionState, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId);

	FAccountInfoRegistryNULL AccountInfoRegistryNULL;
};

/* UE::Online */ }

#undef UE_API
