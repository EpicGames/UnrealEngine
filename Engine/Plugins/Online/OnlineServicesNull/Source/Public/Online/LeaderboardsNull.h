// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Online/LeaderboardsCommon.h"
#include "Containers/List.h"

#define UE_API ONLINESERVICESNULL_API

namespace UE::Online {

class FOnlineServicesNull;

struct FUserScoreNull
{
	FAccountId AccountId;
	uint64 Score;
};

struct FLeaderboardDataNull
{
	FString Name;
	TDoubleLinkedList<FUserScoreNull> UserScoreList;
};

class FLeaderboardsNull : public FLeaderboardsCommon
{
public:
	using Super = FLeaderboardsCommon;

	UE_API FLeaderboardsNull(FOnlineServicesNull& InOwningSubsystem);

	// ILeaderboards
	UE_API virtual TOnlineAsyncOpHandle<FReadEntriesForUsers> ReadEntriesForUsers(FReadEntriesForUsers::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FReadEntriesAroundRank> ReadEntriesAroundRank(FReadEntriesAroundRank::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FReadEntriesAroundUser> ReadEntriesAroundUser(FReadEntriesAroundUser::Params&& Params) override;

	// FLeaderboardsCommon
	UE_API virtual TOnlineAsyncOpHandle<FWriteLeaderboardScores> WriteLeaderboardScores(FWriteLeaderboardScores::Params&& Params) override;

protected:
	TArray<FLeaderboardDataNull> LeaderboardsData;
};

/* UE::Online */ }

#undef UE_API
