// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Online/LobbiesCommon.h"
#include "Online/Lobbies.h"
#include "Online/LANBeacon.h"

class FNboSerializeToBuffer;
class FNboSerializeFromBuffer;

namespace UE::Online {

class FOnlineServicesNull;

class FLobbyNull
{
public: 
	FLobbyNull();

public:
	TSharedRef<FLobby> Data;
	int32 PingInMs;
	TSharedPtr<class FInternetAddr> HostAddr;
	uint32 HostAddrIp;
	int32 HostAddrPort;
};

class FLobbiesNull : public FLobbiesCommon
{

public:

	FLobbiesNull(FOnlineServicesNull& InServices);

	virtual void Initialize() override;
	virtual void PreShutdown() override;
	virtual void Tick(float DeltaSeconds) override;

	virtual TOnlineAsyncOpHandle<FCreateLobby> CreateLobby(FCreateLobby::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FFindLobbies> FindLobbies(FFindLobbies::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FJoinLobby> JoinLobby(FJoinLobby::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FLeaveLobby> LeaveLobby(FLeaveLobby::Params&& Params) override;
	virtual TOnlineResult<FGetJoinedLobbies> GetJoinedLobbies(FGetJoinedLobbies::Params&& Params) override;
private:
	// LAN
	bool NeedsToAdvertise();
	bool IsSessionJoinable( const TSharedRef<FLobbyNull>& Session) const;
	uint32 UpdateLANStatus();
	uint32 JoinLANSession(int32 PlayerNum, TSharedRef<FLobbyNull>& Lobby);
	uint32 FindLANSession();
	uint32 FinalizeLANSearch();
	void AppendLobbyToPacket(FNboSerializeToBuffer& Packet, const TSharedRef<FLobbyNull>& Lobby);
	//void AppendSessionSettingsToPacket(class FNboSerializeToBufferNull& Packet, FOnlineSessionSettings* SessionSettings);
	void ReadLobbyFromPacket(FNboSerializeFromBuffer& Packet, const TSharedRef<FLobbyNull>& Session);
	//void ReadSettingsFromPacket(class FNboSerializeFromBufferNull& Packet, FOnlineSessionSettings& SessionSettings);
	void OnValidQueryPacketReceived(uint8* PacketData, int32 PacketLength, uint64 ClientNonce);
	void OnValidResponsePacketReceived(uint8* PacketData, int32 PacketLength);
	void OnLANSearchTimeout();

private:
	FOnlineServicesNull& Services;
	
	TMap<FName, TSharedRef<FLobbyNull>> NamedLobbies;
	TMap<FLobbyId, TSharedRef<FLobbyNull>> AllLobbies; // todo: this should be moved to the lobby id registry (replication made this tricky to get implemented asap)
	TSharedPtr<FFindLobbies::Result> CurrentLobbySearch;
	TSharedPtr<TOnlineAsyncOp<FFindLobbies>> CurrentLobbySearchHandle;
	FLANSession LANSessionManager;



	// Lobby list management
	TSharedRef<FLobbyNull> CreateNamedLobby(const FCreateLobby::Params& Params);

	TSharedPtr<const FLobbyNull> GetNamedLobby(const FName& LobbyName) const
	{
		const TSharedRef<FLobbyNull>* Lobby = NamedLobbies.Find(LobbyName);
		if (Lobby)
		{
			return *Lobby;
		}
		else
		{
			return nullptr;
		}
	}

	TSharedPtr<FLobbyNull> GetNamedLobby(const FName& LobbyName)
	{
		TSharedRef<FLobbyNull>* Lobby = NamedLobbies.Find(LobbyName);
		if(Lobby)
		{
			return *Lobby;
		}
		else 
		{
			return nullptr;
		}
	}

	TSharedPtr<FLobbyNull> GetLobby(const FLobbyId LobbyId)
	{
		TSharedRef<FLobbyNull>* Lobby = AllLobbies.Find(LobbyId);
		if(Lobby)
		{
			return *Lobby;
		}
		else
		{
			return nullptr;
		}
	}
	
	void AddNamedLobby(const TSharedRef<FLobbyNull>& Lobby, FName Name)
	{
		//checkf(!NamedLobbies.Contains(Name));
		NamedLobbies.Add(Name, Lobby);
		AllLobbies.Add(Lobby->Data->LobbyId, Lobby); // Should be redundant, theoretically, but will help clear up any potential edge cases
		Lobby->Data->LocalName = Name;
	}
	
	// returns true if a named lobby was removed
	bool RemoveLobbyFromRef(const TSharedRef<FLobbyNull>& Lobby)
	{
		return RemoveLobbyFromName(Lobby->Data->LocalName);
	}

	// returns true if a named lobby was removed
	bool RemoveLobbyFromId(const FLobbyId& LobbyId)
	{
		TSharedPtr<FLobbyNull> Lobby = GetLobby(LobbyId);
		if(Lobby)
		{
			int32 NumRemoved = NamedLobbies.Remove(Lobby->Data->LocalName);
			AllLobbies.Remove(LobbyId);
			return NumRemoved > 0;
		}
		return false;
	}
	
	// returns true if a named lobby was removed
	bool RemoveLobbyFromName(const FName& Name)
	{
		TSharedPtr<FLobbyNull> Lobby = GetNamedLobby(Name);
		if(Lobby)
		{
			NamedLobbies.Remove(Name);
			AllLobbies.Remove(Lobby->Data->LobbyId);
			return true;
		}
		return false;
	}

};

} // namespace UE::Online
