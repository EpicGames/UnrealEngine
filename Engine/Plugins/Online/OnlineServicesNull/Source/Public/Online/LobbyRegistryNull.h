// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Online/LobbiesCommon.h"

namespace UE::Online {
using IOnlineLobbyIdRegistry = IOnlineIdRegistry<OnlineIdHandleTags::FLobby>; // todo: remove this when its added to the global scope
class FOnlineLobbyIdRegistryNull : public IOnlineLobbyIdRegistry
{
public:
	ONLINESERVICESNULL_API static FOnlineLobbyIdRegistryNull& Get();

	ONLINESERVICESNULL_API const FLobbyId* Find(FString LobbyIdStr);
	ONLINESERVICESNULL_API FLobbyId FindOrAdd(FString LobbyIdStr);
	ONLINESERVICESNULL_API FLobbyId GetNext();

	// Begin IOnlineAccountIdRegistry
	virtual FString ToString(const FLobbyId& LobbyId) const override;
	virtual FString ToLogString(const FLobbyId& LobbyId) const override;
	virtual TArray<uint8> ToReplicationData(const FLobbyId& LobbyId) const override;
	virtual FLobbyId FromReplicationData(const TArray<uint8>& ReplicationString) override;
	virtual FLobbyId FromStringData(const FString& StringData) override;
	// End IOnlineAccountIdRegistry

	virtual ~FOnlineLobbyIdRegistryNull() = default;

private:
	// TODO use TOnlineBasicIdRegistry
	const FString* GetInternal(const FLobbyId& LobbyId) const;
	TArray<FString> Ids;
	TMap<FString, FLobbyId> StringToId;
};

} // namespace UE::Online