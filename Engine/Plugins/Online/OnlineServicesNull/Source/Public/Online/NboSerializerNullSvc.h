// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/NboSerializer.h"

#include "Online/AuthNull.h"
#include "Online/LobbiesNull.h"
#include "Online/SessionsNull.h"
#include "Online/CoreOnline.h"
#include "Online/NboSerializerCommonSvc.h"

/**
 * Serializes data in network byte order form into a buffer
 */
namespace UE::Online {

namespace NboSerializerNullSvc {

/** NboSerializeToBuffer methods */

inline void SerializeToBuffer(FNboSerializeToBuffer& Ar, const FAccountId& UniqueId)
{
	TArray<uint8> Data = FOnlineAccountIdRegistryNull::Get().ToReplicationData(UniqueId);
	Ar << Data.Num();
	Ar.WriteBinary(Data.GetData(), Data.Num());
}

inline void SerializeToBuffer(FNboSerializeToBuffer& Ar, const FOnlineSessionId& SessionId)
{
	TArray<uint8> Data = FOnlineSessionIdRegistryNull::Get().ToReplicationData(SessionId);
	Ar << Data.Num();
	Ar.WriteBinary(Data.GetData(), Data.Num());
}

inline void SerializeToBuffer(FNboSerializeToBuffer& Packet, const FSessionMemberIdsSet& SessionMembersSet)
{
	Packet << SessionMembersSet.Num();

	for (const FAccountId& SessionMember : SessionMembersSet)
	{
		SerializeToBuffer(Packet, SessionMember);
	}
}

inline void SerializeToBuffer(FNboSerializeToBuffer& Ar, const TMap<FSchemaAttributeId, FSchemaVariant>& Map)
{
	Ar << Map.Num();
	for (const TPair<FSchemaAttributeId, FSchemaVariant>& Pair : Map)
	{
		Ar << Pair.Key;
		NboSerializerCommonSvc::SerializeToBuffer(Ar, Pair.Value);
	}
}

/** NboSerializeFromBuffer methods */

inline void SerializeFromBuffer(FNboSerializeFromBuffer& Ar, FAccountId& UniqueId)
{
	TArray<uint8> Data;
	int32 Size;
	Ar >> Size;
	Ar.ReadBinaryArray(Data, Size);

	UniqueId = FOnlineAccountIdRegistryNull::Get().FromReplicationData(Data);
}

inline void SerializeFromBuffer(FNboSerializeFromBuffer& Ar, FOnlineSessionId& SessionId)
{
	TArray<uint8> Data;
	int32 Size;
	Ar >> Size;
	Ar.ReadBinaryArray(Data, Size);

	SessionId = FOnlineSessionIdRegistryNull::Get().FromReplicationData(Data);
}

inline void SerializeFromBuffer(FNboSerializeFromBuffer& Packet, FSessionMemberIdsSet& SessionMembersSet)
{
	int32 NumEntries = 0;
	Packet >> NumEntries;

	for (int32 Index = 0; Index < NumEntries; ++Index)
	{
		FAccountId Key;
		SerializeFromBuffer(Packet, Key);

		SessionMembersSet.Emplace(Key);
	}
}

inline void SerializeFromBuffer(FNboSerializeFromBuffer& Ar, TMap<FSchemaAttributeId, FSchemaVariant>& Map)
{
	int32 Size;
	Ar >> Size;
	for (int i = 0; i < Size; i++)
	{
		FSchemaAttributeId LobbyAttributeId;
		FSchemaVariant LobbyData;
		Ar >> LobbyAttributeId;
		NboSerializerCommonSvc::SerializeFromBuffer(Ar, LobbyData);
		Map.Emplace(MoveTemp(LobbyAttributeId), MoveTemp(LobbyData));
	}
}

/* NboSerializerNullSvc */ }

/* UE::Online */ }