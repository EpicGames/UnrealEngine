// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineServicesCommon.h"

#define UE_API ONLINESERVICESNULL_API

namespace UE::Online {

class FOnlineServicesNull : public FOnlineServicesCommon
{
public:
	using Super = FOnlineServicesCommon;

	UE_API FOnlineServicesNull(FName InInstanceName, FName InInstanceConfigName);
	UE_API virtual void RegisterComponents() override;
	UE_API virtual void Initialize() override;
	UE_API virtual void PreShutdown() override;
	UE_API virtual TOnlineResult<FGetResolvedConnectString> GetResolvedConnectString(FGetResolvedConnectString::Params&& Params) override;
	virtual EOnlineServices GetServicesProvider() const override { return EOnlineServices::Null; }

};

/* UE::Online */ }

#undef UE_API
