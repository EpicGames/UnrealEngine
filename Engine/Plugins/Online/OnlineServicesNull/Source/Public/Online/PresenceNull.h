// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Online/PresenceCommon.h"

namespace UE::Online {

class FOnlineServicesNull;

class FPresenceNull : public FPresenceCommon
{
public:
	FPresenceNull(FOnlineServicesNull& InServices);

	virtual void Initialize() override;
	virtual void PreShutdown() override;

	virtual TOnlineAsyncOpHandle<FQueryPresence> QueryPresence(FQueryPresence::Params&& Params) override;
	virtual TOnlineResult<FGetCachedPresence> GetCachedPresence(FGetCachedPresence::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FUpdatePresence> UpdatePresence(FUpdatePresence::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FPartialUpdatePresence> PartialUpdatePresence(FPartialUpdatePresence::Params&& Params) override;

protected:
	TMap<FAccountId, TSharedRef<const FUserPresence>> Presences;
	TMap<FAccountId, TSet<FAccountId>> PresenceListeners;
};

	/* UE::Online */
}
