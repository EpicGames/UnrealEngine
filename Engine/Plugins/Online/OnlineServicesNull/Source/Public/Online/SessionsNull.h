// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Online/SessionsLAN.h"

namespace UE::Online {

class FOnlineServicesNull;

class FOnlineSessionIdRegistryNull : public FOnlineSessionIdRegistryLAN
{
public:
	static FOnlineSessionIdRegistryNull& Get();

private:
	FOnlineSessionIdRegistryNull();
};

typedef FSessionLAN FSessionNull;

class FSessionsNull : public FSessionsLAN
{
public:
	using Super = FSessionsLAN;

	FSessionsNull(FOnlineServicesNull& InServices);
	virtual ~FSessionsNull() = default;

private:
	// FSessionsLAN
	virtual void AppendSessionToPacket(FNboSerializeToBuffer& Packet, const FSessionLAN& Session) override;
	virtual void ReadSessionFromPacket(FNboSerializeFromBuffer& Packet, FSessionLAN& Session) override;
};

/* UE::Online */ }