// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/StatsCommon.h"

#define UE_API ONLINESERVICESNULL_API

namespace UE::Online {

class FOnlineServicesNull;

class FStatsNull : public FStatsCommon
{
public:
	using Super = FStatsCommon;

	UE_API FStatsNull(FOnlineServicesNull& InOwningSubsystem);
};

/* UE::Online */ }

#undef UE_API
