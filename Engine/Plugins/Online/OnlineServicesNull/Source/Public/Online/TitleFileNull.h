// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Online/TitleFileCommon.h"

#define UE_API ONLINESERVICESNULL_API

namespace UE::Online {

class FOnlineServicesNull;

class FTitleFileNull : public FTitleFileCommon
{
public:
	using Super = FTitleFileCommon;

	UE_API FTitleFileNull(FOnlineServicesNull& InOwningSubsystem);

	// IOnlineComponent
	UE_API virtual void UpdateConfig() override;

	// ITitleFile
	UE_API virtual TOnlineAsyncOpHandle<FTitleFileEnumerateFiles> EnumerateFiles(FTitleFileEnumerateFiles::Params&& Params) override;
	UE_API virtual TOnlineResult<FTitleFileGetEnumeratedFiles> GetEnumeratedFiles(FTitleFileGetEnumeratedFiles::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FTitleFileReadFile> ReadFile(FTitleFileReadFile::Params&& Params) override;

protected:
	TMap<FString, FTitleFileContentsRef> TitleFiles;

	bool bEnumerated = false;
};

/* UE::Online */ }

#undef UE_API
