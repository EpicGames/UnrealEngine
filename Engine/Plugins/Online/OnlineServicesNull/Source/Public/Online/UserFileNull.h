// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Online/UserFileCommon.h"

#define UE_API ONLINESERVICESNULL_API

namespace UE::Online {

class FOnlineServicesNull;

class FUserFileNull : public FUserFileCommon
{
public:
	using Super = FUserFileCommon;

	UE_API FUserFileNull(FOnlineServicesNull& InOwningSubsystem);

	// IOnlineComponent
	UE_API virtual void UpdateConfig() override;

	// IUserFile
	UE_API virtual TOnlineAsyncOpHandle<FUserFileEnumerateFiles> EnumerateFiles(FUserFileEnumerateFiles::Params&& Params) override;
	UE_API virtual TOnlineResult<FUserFileGetEnumeratedFiles> GetEnumeratedFiles(FUserFileGetEnumeratedFiles::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FUserFileReadFile> ReadFile(FUserFileReadFile::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FUserFileWriteFile> WriteFile(FUserFileWriteFile::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FUserFileCopyFile> CopyFile(FUserFileCopyFile::Params&& Params) override;
	UE_API virtual TOnlineAsyncOpHandle<FUserFileDeleteFile> DeleteFile(FUserFileDeleteFile::Params&& Params) override;

protected:
	using FUserFileMap = TMap<FString, FUserFileContentsRef>;

	struct FUserState
	{
		bool bEnumerated = false;
		FUserFileMap Files;
	};

	TMap<FAccountId, FUserState> UserStates;
	UE_API FUserState& GetUserState(const FAccountId AccountId);

	FUserFileMap InitialFileStateFromConfig;
};

/* UE::Online */ }

#undef UE_API
